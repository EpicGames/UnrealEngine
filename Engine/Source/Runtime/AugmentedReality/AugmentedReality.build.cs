// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class AugmentedReality : ModuleRules
    {
        public AugmentedReality(ReadOnlyTargetRules Target) : base(Target)
        {
            PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "CoreUObject",
                    "Engine",
                    "EngineSettings",
                    "RenderCore",
                    "RHI"
                }
           );
		   
		   if (Target.bBuildEditor)
		   {
			  PrivateDependencyModuleNames.Add("UnrealEd");
		   }

            PublicDependencyModuleNames.AddRange(
                new string[]
                {
                    "MRMesh"
                }
            );

            PublicIncludePathModuleNames.AddRange(
                new string[]
                {
                    "HeadMountedDisplay",
                }
           );

        }
    }
}
