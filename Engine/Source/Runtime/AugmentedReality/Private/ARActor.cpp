// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARActor.h"
#include "ARLifeCycleComponent.h"
#include "ARComponent.h"
#include "AugmentedRealityModule.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARActor)

#if WITH_EDITOR
#include "Editor.h"
#endif


AARActor::AARActor()
{
	bReplicates = true;
	bAlwaysRelevant = true; 
}

UARComponent* AARActor::AddARComponent(TSubclassOf<UARComponent> InComponentClass, const FGuid& NativeID)
{
	if (!InComponentClass)
	{
		return nullptr;
	}
	
	UARComponent* ARComponent = NewObject<UARComponent>(this, InComponentClass);
	ARComponent->SetNativeID(NativeID);
	ARComponent->SetIsReplicated(true);
	ARComponent->SetupAttachment(GetRootComponent());
	ARComponent->RegisterComponent();
	return ARComponent;
}

void AARActor::RequestSpawnARActor(FGuid NativeID, UClass* InComponentClass)
{
	//if the user has signed up for networking, rely on them to spawn the object on the server
	if (UARLifeCycleComponent::RequestSpawnARActorDelegate.IsBound())
	{
		UARLifeCycleComponent::RequestSpawnARActorDelegate.Broadcast(InComponentClass, NativeID);
	}
	else
	{
		//just create these locally and know that networking will not work
		UWorld* World = nullptr;
		if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
		{
			World = GameEngine->GetGameWorld();
		}
		
#if WITH_EDITOR
		if (!World && GEditor)
		{
			for (const auto& Context : GEditor->GetWorldContexts())
			{
				if (Context.WorldType == EWorldType::PIE && Context.World())
				{
					World = Context.World();
					break;
				}
			}
		}
#endif
		if (World)
		{
			const auto SpawnTransform = FTransform::Identity;
			if (auto TempARActor = Cast<AARActor>(World->SpawnActor(AARActor::StaticClass(), &SpawnTransform)))
			{
				//force replication on for all (for now)
				TempARActor->SetReplicates(true);
				TempARActor->AddARComponent(InComponentClass, NativeID);
			}
		}
		else
		{
			UE_LOGF(LogAR, Log, "Failed to find world when trying to spawn ARActor!");
		}
	}
}

void AARActor::RequestDestroyARActor(AARActor* InActor)
{
	if (!InActor)
	{
		return;
	}
	
	if (UARLifeCycleComponent::RequestDestroyARActorDelegate.IsBound())
	{
		UARLifeCycleComponent::RequestDestroyARActorDelegate.Broadcast(InActor);
	}
	else
	{
		// Make sure the actor is properly destroyed in local-only environment
		InActor->Destroy();
	}
}

FTrackedGeometryGroup::FTrackedGeometryGroup(UARTrackedGeometry* InTrackedGeometry)
{
	TrackedGeometry = InTrackedGeometry;
}

