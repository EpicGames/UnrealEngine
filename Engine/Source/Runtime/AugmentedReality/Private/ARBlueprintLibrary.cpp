// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARBlueprintLibrary.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "Engine/Engine.h"
#include "ARPin.h"
#include "ARGeoTrackingSupport.h"
#include "IXRTrackingSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARBlueprintLibrary)


TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> UARBlueprintLibrary::RegisteredARSystem = nullptr;

EARTrackingQuality UARBlueprintLibrary::GetTrackingQuality()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetTrackingQuality();
	}
	else
	{
		return EARTrackingQuality::NotTracking;
	}
}

EARTrackingQualityReason UARBlueprintLibrary::GetTrackingQualityReason()
{
	auto ARSystem = GetARSystem();
 	if (ARSystem.IsValid())
 	{
 		return ARSystem.Pin()->GetTrackingQualityReason();
 	}
 	else
	{
		return EARTrackingQualityReason::InsufficientFeatures;
	}
}

bool UARBlueprintLibrary::IsARSupported(void)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->IsARAvailable();
	}
	return false;
}

void UARBlueprintLibrary::StartARSession(UARSessionConfig* SessionConfig)
{
	if (SessionConfig == nullptr)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		static const TCHAR NoSession_Warning[] = TEXT("Attempting to start an AR session without a session config object");
		GEngine->AddOnScreenDebugMessage(INDEX_NONE, 3600.0f, FColor(255,48,16), NoSession_Warning);
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		return;
	}

	static const TCHAR NotARApp_Warning[] = TEXT("Attempting to start an AR session but there is no AR plugin configured. To use AR, enable the proper AR plugin in the Plugin Settings.");
	
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->StartARSession(SessionConfig);
	}
	else
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		GEngine->AddOnScreenDebugMessage(INDEX_NONE, 3600.0f, FColor(255,48,16), NotARApp_Warning);
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	}
}

void UARBlueprintLibrary::PauseARSession()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->PauseARSession();
	}
}

void UARBlueprintLibrary::StopARSession()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->StopARSession();
	}
}

FARSessionStatus UARBlueprintLibrary::GetARSessionStatus()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetARSessionStatus();
	}
	else
	{
		return FARSessionStatus(EARSessionStatus::NotStarted);
	}
}
UARSessionConfig* UARBlueprintLibrary::GetSessionConfig()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return &ARSystem.Pin()->AccessSessionConfig();
	}
	else
	{
		return nullptr;
	}
}


bool UARBlueprintLibrary::ToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->ToggleARCapture(bOnOff, CaptureType);
	}
	return false;
}

void UARBlueprintLibrary::SetEnabledXRCamera(bool bOnOff)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->SetEnabledXRCamera(bOnOff);
	}
}

FIntPoint UARBlueprintLibrary::ResizeXRCamera(const FIntPoint & InSize)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->ResizeXRCamera(InSize);
	}
	return FIntPoint(0, 0);
}


void UARBlueprintLibrary::SetAlignmentTransform( const FTransform& InAlignmentTransform )
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->SetAlignmentTransform( InAlignmentTransform );
	}
}


TArray<FARTraceResult> UARBlueprintLibrary::LineTraceTrackedObjects( const FVector2D ScreenCoord, bool bTestFeaturePoints, bool bTestGroundPlane, bool bTestPlaneExtents, bool bTestPlaneBoundaryPolygon )
{
	TArray<FARTraceResult> Result;
	
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		EARLineTraceChannels ActiveTraceChannels =
		(bTestFeaturePoints ? EARLineTraceChannels::FeaturePoint : EARLineTraceChannels::None) |
			(bTestGroundPlane ? EARLineTraceChannels::GroundPlane : EARLineTraceChannels::None) |
			(bTestPlaneExtents ? EARLineTraceChannels::PlaneUsingExtent : EARLineTraceChannels::None ) |
			(bTestPlaneBoundaryPolygon ? EARLineTraceChannels::PlaneUsingBoundaryPolygon : EARLineTraceChannels::None);
		
		Result = ARSystem.Pin()->LineTraceTrackedObjects(ScreenCoord, ActiveTraceChannels);
	}
	
	return Result;
}

TArray<FARTraceResult> UARBlueprintLibrary::LineTraceTrackedObjects3D(const FVector Start, const FVector End, bool bTestFeaturePoints, bool bTestGroundPlane, bool bTestPlaneExtents, bool bTestPlaneBoundaryPolygon)
{
	TArray<FARTraceResult> Result;

	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		EARLineTraceChannels ActiveTraceChannels =
			(bTestFeaturePoints ? EARLineTraceChannels::FeaturePoint : EARLineTraceChannels::None) |
			(bTestGroundPlane ? EARLineTraceChannels::GroundPlane : EARLineTraceChannels::None) |
			(bTestPlaneExtents ? EARLineTraceChannels::PlaneUsingExtent : EARLineTraceChannels::None) |
			(bTestPlaneBoundaryPolygon ? EARLineTraceChannels::PlaneUsingBoundaryPolygon : EARLineTraceChannels::None);

		Result = ARSystem.Pin()->LineTraceTrackedObjects(Start, End, ActiveTraceChannels);
	}

	return Result;
}

TArray<UARTrackedGeometry*> UARBlueprintLibrary::GetAllGeometries()
{
	TArray<UARTrackedGeometry*> Geometries;
	
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		Geometries = ARSystem.Pin()->GetAllTrackedGeometries();
	}
	return Geometries;
}

TArray<UARTrackedGeometry*> UARBlueprintLibrary::GetAllGeometriesByClass(TSubclassOf<UARTrackedGeometry> GeometryClass)
{
	if (!GeometryClass)
	{
		return {};
	}
	
	TArray<UARTrackedGeometry*> Geometries;
	for (auto Geometry : GetAllGeometries())
	{
		if (Geometry && Geometry->GetClass()->IsChildOf(GeometryClass))
		{
			Geometries.Add(Geometry);
		}
	}
	return MoveTemp(Geometries);
}

TArray<UARPin*> UARBlueprintLibrary::GetAllPins()
{
	TArray<UARPin*> Pins;
	
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		Pins = ARSystem.Pin()->GetAllPins();
	}
	return Pins;
}

bool UARBlueprintLibrary::IsSessionTypeSupported(EARSessionType SessionType)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->IsSessionTypeSupported(SessionType);
	}
	return false;
}


void UARBlueprintLibrary::DebugDrawTrackedGeometry( UARTrackedGeometry* TrackedGeometry, UObject* WorldContextObject, FLinearColor Color, float OutlineThickness, float PersistForSeconds )
{
	UWorld* MyWorld = WorldContextObject->GetWorld();
	if (TrackedGeometry != nullptr && MyWorld != nullptr)
	{
		TrackedGeometry->DebugDraw(MyWorld, Color, OutlineThickness, PersistForSeconds);
	}
}

void UARBlueprintLibrary::DebugDrawPin( UARPin* ARPin, UObject* WorldContextObject, FLinearColor Color, float Scale, float PersistForSeconds )
{
	UWorld* MyWorld = WorldContextObject->GetWorld();
	if (ARPin != nullptr && MyWorld != nullptr)
	{
		ARPin->DebugDraw(MyWorld, Color, Scale, PersistForSeconds);
	}
}


UARLightEstimate* UARBlueprintLibrary::GetCurrentLightEstimate()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetCurrentLightEstimate();
	}
	return nullptr;
}

UARPin* UARBlueprintLibrary::PinComponent( USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry, const FName DebugName )
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->PinComponent( ComponentToPin, PinToWorldTransform, TrackedGeometry, DebugName );
	}
	return nullptr;
}

UARPin* UARBlueprintLibrary::PinComponentToTraceResult( USceneComponent* ComponentToPin, const FARTraceResult& TraceResult, const FName DebugName )
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->PinComponent( ComponentToPin, TraceResult, DebugName );
	}
	return nullptr;
}

bool UARBlueprintLibrary::PinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin)
{
	auto ARSystem = GetARSystem();
	if (!ARSystem.IsValid())
	{
		return false;
	}
	return ARSystem.Pin()->PinComponent(ComponentToPin, Pin);
}

void UARBlueprintLibrary::UnpinComponent( USceneComponent* ComponentToUnpin )
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		auto PinnedARSystem = ARSystem.Pin();
		TArray<UARPin*> AllPins = PinnedARSystem->GetAllPins();
		const int32 AllPinsCount = AllPins.Num();
		for (int32 i=0; i<AllPinsCount; ++i)
		{
			if (AllPins[i]->GetPinnedComponent() == ComponentToUnpin)
			{
				PinnedARSystem->RemovePin( AllPins[i] );
				return;
			}
		}
	}
}

void UARBlueprintLibrary::RemovePin( UARPin* PinToRemove )
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->RemovePin( PinToRemove );
	}
}

bool UARBlueprintLibrary::IsARPinLocalStoreSupported()
{
	bool bSuccess = false;

	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		bSuccess = ARSystem.Pin()->IsLocalPinSaveSupported();
	}

	return bSuccess;
}

bool UARBlueprintLibrary::IsARPinLocalStoreReady()
{
	bool bSuccess = false;

	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		bSuccess = ARSystem.Pin()->ArePinsReadyToLoad();
	}

	return bSuccess;
}

TMap<FName, UARPin*> UARBlueprintLibrary::LoadARPinsFromLocalStore()
{
	TMap<FName, UARPin*> LoadedPins;

	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->LoadARPins(LoadedPins);
	}

	return LoadedPins;
}

bool UARBlueprintLibrary::SaveARPinToLocalStore(FName InSaveName, UARPin* InPin)
{
	bool bSuccess = false;

	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		bSuccess = ARSystem.Pin()->SaveARPin(InSaveName, InPin);
	}

	return bSuccess;
}

void UARBlueprintLibrary::RemoveARPinFromLocalStore(FName InSaveName)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->RemoveSavedARPin(InSaveName);
	}
}

void UARBlueprintLibrary::RemoveAllARPinsFromLocalStore()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		ARSystem.Pin()->RemoveAllSavedARPins();
	}
}


void UARBlueprintLibrary::RegisterAsARSystem(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& NewARSystem)
{
	RegisteredARSystem = NewARSystem;
}


const TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe>& UARBlueprintLibrary::GetARSystem()
{
	return RegisteredARSystem;
}



float UARTraceResultLibrary::GetDistanceFromCamera( const FARTraceResult& TraceResult )
{
	return TraceResult.GetDistanceFromCamera();
}

FTransform UARTraceResultLibrary::GetLocalToTrackingTransform( const FARTraceResult& TraceResult )
{
	return TraceResult.GetLocalToTrackingTransform();
}

FTransform UARTraceResultLibrary::GetLocalToWorldTransform( const FARTraceResult& TraceResult )
{
	return TraceResult.GetLocalToWorldTransform();
}

FTransform UARTraceResultLibrary::GetLocalTransform( const FARTraceResult& TraceResult )
{
	return TraceResult.GetLocalTransform();
}

UARTrackedGeometry* UARTraceResultLibrary::GetTrackedGeometry( const FARTraceResult& TraceResult )
{
	return TraceResult.GetTrackedGeometry();
}

EARLineTraceChannels UARTraceResultLibrary::GetTraceChannel( const FARTraceResult& TraceResult )
{
	return TraceResult.GetTraceChannel();
}

bool UARBlueprintLibrary::AddManualEnvironmentCaptureProbe(FVector Location, FVector Extent)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->AddManualEnvironmentCaptureProbe(Location, Extent);
	}
	return false;
}

EARWorldMappingState UARBlueprintLibrary::GetWorldMappingStatus()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetWorldMappingStatus();
	}
	return EARWorldMappingState::NotAvailable;
}

TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> UARBlueprintLibrary::SaveWorld()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->SaveWorld();
		
	}
	return TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe>();
}

TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> UARBlueprintLibrary::GetCandidateObject(FVector Location, FVector Extent)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetCandidateObject(Location, Extent);
		
	}
	return TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>();
}

TArray<FVector> UARBlueprintLibrary::GetPointCloud()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetPointCloud();
	}
	return TArray<FVector>();
}

TArray<FARVideoFormat> UARBlueprintLibrary::GetSupportedVideoFormats(EARSessionType SessionType)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetSupportedVideoFormats(SessionType);
	}
	return TArray<FARVideoFormat>();
}

UARCandidateImage* UARBlueprintLibrary::AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth)
{
	auto ARSystem = GetARSystem();
	if (ensure(ARSystem.IsValid()))
	{
		return ARSystem.Pin()->AddRuntimeCandidateImage(SessionConfig, CandidateTexture, FriendlyName, PhysicalWidth);
	}
	else
	{
		return nullptr;
	}
}

bool UARBlueprintLibrary::IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->IsSessionTrackingFeatureSupported(SessionType, SessionTrackingFeature);
	}
	else
	{
		return false;
	}
}

TArray<FARPose2D> UARBlueprintLibrary::GetAllTracked2DPoses()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetTracked2DPose();
	}
	else
	{
		return {};
	}
}

bool UARBlueprintLibrary::IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->IsSceneReconstructionSupported(SessionType, SceneReconstructionMethod);
	}
	else
	{
		return false;
	}
}

bool UARBlueprintLibrary::GetObjectClassificationAtLocation(const FVector& InWorldLocation, EARObjectClassification& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff)
{
	auto AllGeometries = GetAllGeometries();
	for (auto Geometry : AllGeometries)
	{
		if (auto MeshGeometry = Cast<UARMeshGeometry>(Geometry))
		{
			if (MeshGeometry->GetObjectClassificationAtLocation(InWorldLocation, OutClassification, OutClassificationLocation, MaxLocationDiff))
			{
				return true;
			}
		}
	}
	
	return false;
}


namespace UARBlueprintLibraryNamespace
{
	struct FSquareMatrix3
	{
		float M[3][3];

		FSquareMatrix3() = default;

		FORCEINLINE void SetColumn(int32 Index, FVector Axis)
		{
			M[Index][0] = (float) Axis.X;
			M[Index][1] = (float) Axis.Y;
			M[Index][2] = (float) Axis.Z;
		};

		FORCEINLINE FVector GetColumn(int32 Index) const
		{
			return FVector(M[Index][0], M[Index][1], M[Index][2]);
		};

		FORCEINLINE FSquareMatrix3 operator*(float Scale) const
		{
			FSquareMatrix3 ResultMat;

			for (int32 X = 0; X < 3; X++)
			{
				for (int32 Y = 0; Y < 3; Y++)
				{
					ResultMat.M[X][Y] = M[X][Y] * Scale;
				}
			}

			return ResultMat;
		}
	};


	FSquareMatrix3 Invert(const FSquareMatrix3& InMatrix)
	{
		FSquareMatrix3 inverse;
		bool invertible;
		auto& M = InMatrix.M;
		float c00 = M[1][1] * M[2][2] - M[1][2] * M[2][1];
		float c10 = M[1][2] * M[2][0] - M[1][0] * M[2][2];
		float c20 = M[1][0] * M[2][1] - M[1][1] * M[2][0];
		float det = M[0][0] * c00 + M[0][1] * c10 + M[0][2] * c20;
		if (FMath::Abs(det) > 0.000001f)
		{
			float invDet = 1.0f / det;
			inverse.SetColumn(0, { c00 * invDet, (M[0][2] * M[2][1] - M[0][1] * M[2][2]) * invDet, (M[0][1] * M[1][2] - M[0][2] * M[1][1]) * invDet });
			inverse.SetColumn(1, { c10 * invDet, (M[0][0] * M[2][2] - M[0][2] * M[2][0]) * invDet, (M[0][2] * M[1][0] - M[0][0] * M[1][2]) * invDet });
			inverse.SetColumn(2, { c20 * invDet, (M[0][1] * M[2][0] - M[0][0] * M[2][1]) * invDet, (M[0][0] * M[1][1] - M[0][1] * M[1][0]) * invDet });
			invertible = true;
		}
		else
		{
			inverse.SetColumn(0, { 1,0,0 });
			inverse.SetColumn(1, { 0,1,0 });
			inverse.SetColumn(2, { 0,0,1 });
			invertible = false;
		}
		return inverse;
	}

	FVector operator* (const FSquareMatrix3& InMatrix, const FVector& InVector)
	{
		auto& M = InMatrix.M;
		return {
			M[0][0] * InVector.X + M[1][0] * InVector.Y + M[2][0] * InVector.Z,
			M[0][1] * InVector.X + M[1][1] * InVector.Y + M[2][1] * InVector.Z,
			M[0][2] * InVector.X + M[1][2] * InVector.Y + M[2][2] * InVector.Z
		};
	}
}

void UARBlueprintLibrary::CalculateClosestIntersection(const TArray<FVector>& StartPoints, const TArray<FVector>& EndPoints, FVector& ClosestIntersection)
{
	checkf(StartPoints.Num() == EndPoints.Num(), TEXT("The StartPoints and Endpoints arrays must have the same size."));

	float SXX = 0.0f;
	float SYY = 0.0f;
	float SZZ = 0.0f;
	float SXY = 0.0f;
	float SXZ = 0.0f;
	float SYZ = 0.0f;

	float CX = 0.0f;
	float CY = 0.0f;
	float CZ = 0.0f;

	for (int PointIdx = 0; PointIdx < StartPoints.Num(); ++PointIdx)
	{
		FVector NormView = (EndPoints[PointIdx] - StartPoints[PointIdx]).GetSafeNormal();

		float XX = ((float)NormView.X * (float)NormView.X) - 1.0f;
		float YY = ((float)NormView.Y * (float)NormView.Y) - 1.0f;
		float ZZ = ((float)NormView.Z * (float)NormView.Z) - 1.0f;
		float XY = ((float)NormView.X * (float)NormView.Y);
		float XZ = ((float)NormView.X * (float)NormView.Z);
		float YZ = ((float)NormView.Y * (float)NormView.Z);

		SXX += XX;
		SYY += YY;
		SZZ += ZZ;
		SXY += XY;
		SXZ += XZ;
		SYZ += YZ;

		CX += ((float)StartPoints[PointIdx].X * XX) +
			((float)StartPoints[PointIdx].Y * XY) +
			((float)StartPoints[PointIdx].Z * XZ);

		CY += ((float)StartPoints[PointIdx].X * XY) +
			((float)StartPoints[PointIdx].Y * YY) +
			((float)StartPoints[PointIdx].Z * YZ);

		CZ += ((float)StartPoints[PointIdx].X * XZ) +
			((float)StartPoints[PointIdx].Y * YZ) +
			((float)StartPoints[PointIdx].Z * ZZ);
	}
	UARBlueprintLibraryNamespace::FSquareMatrix3 S;
	S.SetColumn(0, { SXX, SXY, SXZ });
	S.SetColumn(1, { SXY, SYY, SYZ });
	S.SetColumn(2, { SXZ, SYZ, SZZ });

	FVector C(CX, CY, CZ);

	ClosestIntersection = UARBlueprintLibraryNamespace::Invert(S) * C;
}

void UARBlueprintLibrary::CalculateAlignmentTransform(const FTransform& TransformInFirstCoordinateSystem, const FTransform& TransformInSecondCoordinateSystem, FTransform& AlignmentTransform)
{
	AlignmentTransform = TransformInSecondCoordinateSystem * TransformInFirstCoordinateSystem.Inverse();
}

void UARBlueprintLibrary::SetARWorldOriginLocationAndRotation(FVector OriginLocation, FRotator OriginRotation, bool bIsTransformInWorldSpace, bool bMaintainUpDirection)
{
	// Zero out the pitch and roll if needed
	if (bMaintainUpDirection)
	{
		OriginRotation.Pitch = 0;
		OriginRotation.Roll = 0;
	}
	
	// For P in local space we have T (local) * T (Alignment) * T (tracking to world) = T (World)
	// The goal is to compute a new alignment transform so that T (local) * T (New Alignment) * T (tracking to world) = T (Identity)
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		if (auto TrackingSystem = ARSystem.Pin()->GetXRTrackingSystem())
		{
			const auto TrackingToWorldTransform = TrackingSystem->GetTrackingToWorldTransform();
			const auto TrackingToWorldTransformInverse = TrackingToWorldTransform.Inverse();
			const auto AlignmentTransform = GetAlignmentTransform();
			FTransform LocalTransform;
			if (bIsTransformInWorldSpace)
			{
				const FTransform WorldTransform(OriginRotation, OriginLocation);
				LocalTransform = WorldTransform * TrackingToWorldTransformInverse * AlignmentTransform.Inverse();
			}
			else
			{
				// Note that the local transform has the inverse scale of the alignment transform
				// this makes sure the scale of the alignment transform is unchanged from the calculation below
				LocalTransform = FTransform(OriginRotation, OriginLocation, FVector::OneVector / AlignmentTransform.GetScale3D());
			}
			// now we have the local transform, from T (local) * T (New Alignment) * T (tracking to world) = T (Identity), we have
			// T (New Alignment) = T (inverse local) * T (Identity) * T (inverse tracking to world)
			auto NewAlignmentTransform = LocalTransform.Inverse() * TrackingToWorldTransformInverse;
			if (bMaintainUpDirection)
			{
				auto Rotator = NewAlignmentTransform.Rotator();
				Rotator.Pitch = 0;
				Rotator.Roll = 0;
				NewAlignmentTransform.SetRotation(Rotator.Quaternion());
			}
			auto Rotator = NewAlignmentTransform.Rotator();
			SetAlignmentTransform(NewAlignmentTransform);
		}
	}
}

void UARBlueprintLibrary::SetARWorldScale(float InWorldScale)
{
	if (ensure(InWorldScale > 0.f))
	{
		auto ARSystem = GetARSystem();
		if (ARSystem.IsValid())
		{
			if (auto TrackingSystem = ARSystem.Pin()->GetXRTrackingSystem())
			{
				const auto AlignmentTransform = GetAlignmentTransform();
				FTransform NewAlignmentTransform(AlignmentTransform.GetRotation(), AlignmentTransform.GetLocation(), FVector(1.f / InWorldScale));
				SetAlignmentTransform(NewAlignmentTransform);
				
				const auto TrackingToWorldTransform = TrackingSystem->GetTrackingToWorldTransform();
				const auto TrackingToWorldTransformInverse = TrackingToWorldTransform.Inverse();
				
				// For P in local space whose world space transform is at the origin
				// We have T (local) * T (Alignment) * T (tracking to world) = T (Identity)
				// yields T (local) = T (inverse tracking to world) * T (inverse alignment)
				const auto LocalTransform = TrackingToWorldTransform.Inverse() * AlignmentTransform.Inverse();
				
				// We want to make sure the new alignment transform still puts P at the origin after the world scale change
				// So we calculate where P is after the alignment transform change and move the world origin to it
				// const auto WorldTransform = InverseAlignmentTransform * NewAlignmentTransform;
				SetARWorldOriginLocationAndRotation(LocalTransform.GetLocation(), LocalTransform.Rotator(), false, false);
			}
		}
	}
}

float UARBlueprintLibrary::GetARWorldScale()
{
	const auto AlignmentTransform = GetAlignmentTransform();
	// Assuming the scale is uniform
	return 1.f / (float)AlignmentTransform.GetScale3D().X;
}

FTransform UARBlueprintLibrary::GetAlignmentTransform()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetAlignmentTransform();
	}
	return FTransform::Identity;
}

bool UARBlueprintLibrary::AddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid() && PointName.Len())
	{
		return ARSystem.Pin()->AddTrackedPointWithName(WorldTransform, PointName, bDeletePointsWithSameName);
	}
	
	return false;
}

TArray<UARTrackedPoint*> UARBlueprintLibrary::FindTrackedPointsByName(const FString& PointName)
{
	TArray<UARTrackedPoint*> Points;
	
	for (auto Point : GetAllGeometriesByClass<UARTrackedPoint>())
	{
		if (Point && Point->GetName() == PointName)
		{
			Points.Add(Point);
		}
	}
	
	return MoveTemp(Points);
}

int32 UARBlueprintLibrary::GetNumberOfTrackedFacesSupported()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetNumberOfTrackedFacesSupported();
	}
	return 0;
}

UARTexture* UARBlueprintLibrary::GetARTexture(EARTextureType TextureType)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetARTexture(TextureType);
	}
	return nullptr;
}

bool UARBlueprintLibrary::GetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics)
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		return ARSystem.Pin()->GetCameraIntrinsics(OutCameraIntrinsics);		
	}
	return false;
}

