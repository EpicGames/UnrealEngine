// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARBlueprintProxy.h"
#include "ARSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARBlueprintProxy)

TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> UARBaseAsyncTaskBlueprintProxy::RegisteredARSystem = nullptr;

UARBaseAsyncTaskBlueprintProxy::UARBaseAsyncTaskBlueprintProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShouldTick(true)
{
}

void UARBaseAsyncTaskBlueprintProxy::Tick(float DeltaTime)
{
	if (!AsyncTask.IsValid())
	{
		bShouldTick = false;
		ReportFailure();
		return;
	}

	if (AsyncTask->IsDone())
	{
		bShouldTick = false;
		// Fire the right delegate
		if (!AsyncTask->HadError())
		{
			ReportSuccess();
		}
		else
		{
			ReportFailure();
		}
	}
}

void UARBaseAsyncTaskBlueprintProxy::RegisterAsARSystem(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& NewARSystem)
{
	RegisteredARSystem = NewARSystem;
}


const TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe>& UARBaseAsyncTaskBlueprintProxy::GetARSystem()
{
	return RegisteredARSystem;
}

UARSaveWorldAsyncTaskBlueprintProxy* UARSaveWorldAsyncTaskBlueprintProxy::ARSaveWorld(UObject* WorldContextObject)
{
	UARSaveWorldAsyncTaskBlueprintProxy* Proxy = NewObject<UARSaveWorldAsyncTaskBlueprintProxy>();
	Proxy->RegisterWithGameInstance(WorldContextObject);
	return Proxy;
}

void UARSaveWorldAsyncTaskBlueprintProxy::Activate()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		SaveWorldTask = ARSystem.Pin()->SaveWorld();
		AsyncTask = SaveWorldTask;
	}
	else
	{
		AsyncTask = MakeShared<FARErrorSaveWorldAsyncTask, ESPMode::ThreadSafe>(TEXT("ARSaveWorld - requires a valid, running session"));
	}
}

void UARSaveWorldAsyncTaskBlueprintProxy::ReportSuccess()
{
	OnSuccess.Broadcast(SaveWorldTask->GetSavedWorldData());
}

void UARSaveWorldAsyncTaskBlueprintProxy::ReportFailure()
{
	OnFailed.Broadcast(TArray<uint8>());
}

UARGetCandidateObjectAsyncTaskBlueprintProxy* UARGetCandidateObjectAsyncTaskBlueprintProxy::ARGetCandidateObject(UObject* WorldContextObject, FVector Location, FVector Extent)
{
	UARGetCandidateObjectAsyncTaskBlueprintProxy* Proxy = NewObject<UARGetCandidateObjectAsyncTaskBlueprintProxy>();
	Proxy->RegisterWithGameInstance(WorldContextObject);
	Proxy->Extent = Extent;
	Proxy->Location = Location;

	return Proxy;
}

void UARGetCandidateObjectAsyncTaskBlueprintProxy::Activate()
{
	auto ARSystem = GetARSystem();
	if (ARSystem.IsValid())
	{
		CandidateObjectTask = ARSystem.Pin()->GetCandidateObject(Location, Extent);
		AsyncTask = CandidateObjectTask;
	}
	else
	{
		AsyncTask = MakeShared<FARErrorGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>(TEXT("ARGetCandidateObject - requires a valid, running session"));
	}
}

void UARGetCandidateObjectAsyncTaskBlueprintProxy::ReportSuccess()
{
	OnSuccess.Broadcast(CandidateObjectTask->GetCandidateObject());
}

void UARGetCandidateObjectAsyncTaskBlueprintProxy::ReportFailure()
{
	OnFailed.Broadcast(nullptr);
}

