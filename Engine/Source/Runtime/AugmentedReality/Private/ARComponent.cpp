// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARComponent.h"
#include "ARActor.h"
#include "ARLifeCycleComponent.h"
#include "ARTrackable.h"
#include "ARSessionConfig.h"
#include "AugmentedRealityModule.h"
#include "Net/UnrealNetwork.h"
#include "MRMeshComponent.h"
#include "Kismet/KismetSystemLibrary.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARComponent)


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	#define AR_DEBUG_MODE 1
#else
	#define AR_DEBUG_MODE 0
#endif

DECLARE_STATS_GROUP(TEXT("ARComponent"), STATGROUP_ARComponent, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Update FaceComponent"), STAT_UpdateFaceComponent, STATGROUP_ARComponent);
DECLARE_CYCLE_STAT(TEXT("Calculate Vertex Normals"), STAT_CalculateVertexNormals, STATGROUP_ARComponent);

static TMap<EARObjectClassification, FLinearColor> ClassificationColors =
{
	{ EARObjectClassification::Wall, FLinearColor::Green },
	{ EARObjectClassification::Ceiling, FLinearColor::Blue },
	{ EARObjectClassification::Floor, FLinearColor::Blue },
	{ EARObjectClassification::Table, FLinearColor::Yellow },
	{ EARObjectClassification::Seat, FLinearColor::Yellow },
	{ EARObjectClassification::Door, FLinearColor::Red },
};

template <class FReal>
void NotifyDebugModeUpdated()
{
#if AR_DEBUG_MODE
	if (UKismetSystemLibrary::IsDedicatedServer(nullptr))
	{
		return;
	}
	
	for (TObjectIterator<FReal> ComponentItr; ComponentItr; ++ComponentItr)
	{
		auto Component = *ComponentItr;
		if (Component->GetOwner())
		{
			Component->UpdateVisualization();
		}
	}
#endif
}

#define DEFINE_AR_COMPONENT_DEBUG_MODE(VariableName,CVarName,CVarDescription,ComponentClass) \
static int32 G##VariableName = 0; \
static FAutoConsoleVariableRef CVar##VariableName(\
	TEXT(CVarName), \
	G##VariableName, \
	TEXT(CVarDescription), \
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) \
	{ \
		NotifyDebugModeUpdated<U##ComponentClass>(); \
	})); \
int32 U##ComponentClass::GetDebugMode() \
{ \
	return G##VariableName; \
} \
void U##ComponentClass::Set##VariableName(E##VariableName NewDebugMode) \
{ \
	CVar##VariableName->Set((int32)NewDebugMode); \
}

DEFINE_AR_COMPONENT_DEBUG_MODE(PlaneComponentDebugMode, "ar.PlaneComponentDebugMode", "Debug mode for AR plane component, see EPlaneComponentDebugMode", ARPlaneComponent);
DEFINE_AR_COMPONENT_DEBUG_MODE(ImageComponentDebugMode, "ar.ImageComponentDebugMode", "Debug mode for AR image component, see EImageComponentDebugMode", ARImageComponent);
DEFINE_AR_COMPONENT_DEBUG_MODE(FaceComponentDebugMode, "ar.FaceComponentDebugMode", "Debug mode for AR face component, see EFaceComponentDebugMode", ARFaceComponent);
DEFINE_AR_COMPONENT_DEBUG_MODE(PoseComponentDebugMode, "ar.PoseComponentDebugMode", "Debug mode for AR pose component, see EPoseComponentDebugMode", ARPoseComponent);
DEFINE_AR_COMPONENT_DEBUG_MODE(QRCodeComponentDebugMode, "ar.QRCodeComponentDebugMode", "Debug mode for AR QR code component, see EQRCodeComponentDebugMode", ARQRCodeComponent);
DEFINE_AR_COMPONENT_DEBUG_MODE(GeoAnchorComponentDebugMode, "ar.GeoAnchorComponentDebugMode", "Debug mode for AR Geo anchor component, see EGeoAnchorComponentDebugMode", ARGeoAnchorComponent);

UARComponent::UARComponent()
{
	SetIsReplicatedByDefault(true);
	
#if AR_DEBUG_MODE
	PrimaryComponentTick.bCanEverTick = true;
#endif
}

void UARComponent::BeginPlay()
{
	Super::BeginPlay();

	//now make sure the AR system knows this was made
	AARActor* NewActor = Cast<AARActor>(GetOwner());
	UARLifeCycleComponent::OnSpawnARActorDelegate.Broadcast(NewActor, this, NativeID);
}

void UARComponent::SetNativeID(FGuid InNativeID)
{
	NativeID = InNativeID;
}

void UARComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME(UARComponent, NativeID);
}

FLinearColor UARComponent::GetDebugColor() const
{
	return MyTrackedGeometry ? FLinearColor::Green : FLinearColor::Red;
}

#define DEFINE_AR_COMPONENT_VIRTUALS(NetworkClassName,NativeClassName,UpdatePayloadClass) \
void  U##NetworkClassName::Update(UARTrackedGeometry* TrackedGeometry) \
{ \
	check(TrackedGeometry); \
	U##NativeClassName *NativeTrackedGeometry = Cast<U##NativeClassName>(TrackedGeometry); \
	if (NativeTrackedGeometry) \
	{ \
		F##UpdatePayloadClass Payload; \
		NativeTrackedGeometry->GetNetworkPayload(Payload); \
		ReceiveUpdate(Payload); \
		if (bUseDefaultReplication) \
		{ \
			ServerUpdatePayload(Payload); \
			if (GetOwner()->GetLocalRole() != ROLE_Authority) \
			{ \
				ReplicatedPayload = Payload; \
				OnRep_Payload(); \
			} \
		} \
		if (MRMeshComponent) \
		{ \
			NativeTrackedGeometry->SetUnderlyingMesh(MRMeshComponent); \
		} \
		MyTrackedGeometry = TrackedGeometry; \
	} \
	else \
	{ \
		UE_LOGF(LogAR, Warning, "%ls unable to cast to %ls from %ls in Update", *U##NetworkClassName::StaticClass()->GetName(), *U##NativeClassName::StaticClass()->GetName(), *TrackedGeometry->GetName()); \
	} \
} \
void U##NetworkClassName::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const \
{ \
	Super::GetLifetimeReplicatedProps(OutLifetimeProps); \
	DOREPLIFETIME_CONDITION(U##NetworkClassName, ReplicatedPayload, COND_SkipOwner); \
} \
bool U##NetworkClassName::ServerUpdatePayload_Validate(const F##UpdatePayloadClass& NewPayload) \
{ \
	return true; \
} \
void U##NetworkClassName::ServerUpdatePayload_Implementation(const F##UpdatePayloadClass& NewPayload) \
{ \
	ReplicatedPayload = NewPayload; \
	OnRep_Payload(); \
} \
void U##NetworkClassName::OnRep_Payload() \
{ \
	Super::OnRep_Payload(); \
	if (bFirstUpdate) \
	{ \
		ReceiveAdd(ReplicatedPayload); \
		bFirstUpdate = false; \
	} \
	else \
	{ \
		ReceiveUpdate(ReplicatedPayload); \
	} \
}

DEFINE_AR_COMPONENT_VIRTUALS(ARPlaneComponent,            ARPlaneGeometry,           ARPlaneUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARPointComponent,            ARTrackedPoint,            ARPointUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARFaceComponent,             ARFaceGeometry,            ARFaceUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARImageComponent,            ARTrackedImage,            ARImageUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARQRCodeComponent,           ARTrackedQRCode,           ARQRCodeUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARPoseComponent,             ARTrackedPose,             ARPoseUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(AREnvironmentProbeComponent, AREnvironmentCaptureProbe, AREnvironmentProbeUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARObjectComponent,           ARTrackedObject,           ARObjectUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARMeshComponent,             ARTrackedGeometry,         ARMeshUpdatePayload);
DEFINE_AR_COMPONENT_VIRTUALS(ARGeoAnchorComponent,        ARGeoAnchor,               ARGeoAnchorUpdatePayload);


void FARSessionPayload::SetFlag(EARSessionConfigFlags InFlag)
{
	ConfigFlags |= (int32)InFlag;
}

bool FARSessionPayload::HasFlag(EARSessionConfigFlags InFlag) const
{
	return (ConfigFlags & (int32)InFlag) != 0;
}

bool FARSessionPayload::ShouldCreateMeshComponent() const
{
	return HasFlag(EARSessionConfigFlags::GenerateMeshData);
}

void FARSessionPayload::FromSessionConfig(const UARSessionConfig& InConfig)
{
	if (InConfig.bGenerateMeshDataFromTrackedGeometry)
	{
		SetFlag(EARSessionConfigFlags::GenerateMeshData);
	}
	
	if (InConfig.bGenerateCollisionForMeshData)
	{
		SetFlag(EARSessionConfigFlags::GenerateCollisionForMeshData);
	}
	
	if (InConfig.bGenerateNavMeshForMeshData)
	{
		SetFlag(EARSessionConfigFlags::GenerateNavMeshForMeshData);
	}
	
	if (InConfig.bUseMeshDataForOcclusion)
	{
		SetFlag(EARSessionConfigFlags::UseMeshDataForOcclusion);
	}
	
	if (InConfig.bRenderMeshDataInWireframe)
	{
		SetFlag(EARSessionConfigFlags::RenderMeshDataInWireframe);
	}
	
	DefaultMeshMaterial = InConfig.GetDefaultMeshMaterial();

	DefaultWireframeMeshMaterial = InConfig.GetDefaultWireframeMeshMaterial();
}

void UARComponent::OnUnregister()
{
	if (!bIsRemoved)
	{
		ReceiveRemove();
		bIsRemoved = true;
	}
	
	Super::OnUnregister();
}

void UARComponent::Remove(UARTrackedGeometry* TrackedGeometry)
{
	if (!bIsRemoved)
	{
		ReceiveRemove();
		bIsRemoved = true;
	}
	
	RemoveMeshComponent(TrackedGeometry);
}

void UARComponent::RemoveMeshComponent(UARTrackedGeometry* TrackedGeometry)
{
	if (MRMeshComponent)
	{
		OnMRMeshDestroyed.Broadcast(MRMeshComponent);
		MRMeshComponent->UnregisterComponent();
		MRMeshComponent = nullptr;
	}
	
	if (TrackedGeometry)
	{
		TrackedGeometry->SetUnderlyingMesh(nullptr);
	}
}

void UARComponent::UpdateVisualization_Implementation()
{
}

void UARComponent::OnRep_Payload()
{
	if (!UKismetSystemLibrary::IsDedicatedServer(this))
	{
		// Update visualization if we're not on the server
		UpdateVisualization();
	}
}

void UARComponent::ManageMeshComponentForDebugMode(bool bDebugModeEnabled, const FARSessionPayload& SessionPayload)
{
#if AR_DEBUG_MODE
	if (bDebugModeEnabled)
	{
		if (!bInDebugMode)
		{
			// Save the wireframe info when entering debug mode
			bSavedWireframeMode = MRMeshComponent->GetUseWireframe();
			SavedWireframeColor = MRMeshComponent->GetWireframeColor();
			bInDebugMode = true;
		}
		MRMeshComponent->SetUseWireframe(true);
	}
	else
	{
		if (bInDebugMode)
		{
			// Restore the wireframe info when leaving debug mode
			MRMeshComponent->SetUseWireframe(bSavedWireframeMode);
			MRMeshComponent->SetWireframeColor(SavedWireframeColor);
			bInDebugMode = false;
		}
		
		if (!SessionPayload.ShouldCreateMeshComponent())
		{
			// MRMeshComponent was created due to previously enabled debug mode
			// so it should removed now
			RemoveMeshComponent(MyTrackedGeometry);
		}
	}
#endif
}

static UMRMeshComponent* CreateMRMeshComponent(AActor* Owner, const FARSessionPayload& SessionPayload, UMaterialInterface* DefaultMeshMaterial, UMaterialInterface* DefaultWireframeMeshMaterial, bool bDebugModeEnabled = false)
{
#if !AR_DEBUG_MODE
	bDebugModeEnabled = false;
#endif
	
	if (SessionPayload.ShouldCreateMeshComponent() || bDebugModeEnabled)
	{
		auto MRMeshComponent = NewObject<UMRMeshComponent>(Owner);
		
		MRMeshComponent->SetUsingAbsoluteLocation(true);
		MRMeshComponent->SetUsingAbsoluteRotation(true);
		MRMeshComponent->SetUsingAbsoluteScale(true);
		
		MRMeshComponent->SetEnableMeshOcclusion(SessionPayload.HasFlag(EARSessionConfigFlags::UseMeshDataForOcclusion));
		MRMeshComponent->SetUseWireframe(SessionPayload.HasFlag(EARSessionConfigFlags::RenderMeshDataInWireframe));
		MRMeshComponent->SetNeverCreateCollisionMesh(!SessionPayload.HasFlag(EARSessionConfigFlags::GenerateCollisionForMeshData));
		MRMeshComponent->SetEnableNavMesh(SessionPayload.HasFlag(EARSessionConfigFlags::GenerateNavMeshForMeshData));
		
		if (!DefaultMeshMaterial)
		{
			DefaultMeshMaterial = SessionPayload.DefaultMeshMaterial;
		}
		
		if (DefaultMeshMaterial)
		{
			MRMeshComponent->SetMaterial(0, DefaultMeshMaterial);
		}
		
		if (!DefaultWireframeMeshMaterial)
		{
			DefaultWireframeMeshMaterial = SessionPayload.DefaultWireframeMeshMaterial;
		}

		if (DefaultWireframeMeshMaterial)
		{
			MRMeshComponent->SetWireframeMaterial(DefaultWireframeMeshMaterial);
		}
		
		MRMeshComponent->SetupAttachment(Owner->GetRootComponent());
		MRMeshComponent->RegisterComponent();
		
		return MRMeshComponent;
	}
	
	return nullptr;
}

static void UpdatePlaneMesh(UMRMeshComponent* MeshComponent, const FTransform& WorldTransform, const FVector& Center, const FVector& Extent)
{
	check(MeshComponent);
	
	TArray<FVector> Vertices;
	Vertices.Reset(4);
	Vertices.Add(Center + FVector(Extent.X, Extent.Y, 0.f));
	Vertices.Add(Center + FVector(Extent.X, -Extent.Y, 0.f));
	Vertices.Add(Center + FVector(-Extent.X, -Extent.Y, 0.f));
	Vertices.Add(Center + FVector(-Extent.X, Extent.Y, 0.f));
	TArray<MRMESH_INDEX_TYPE> Indices = { 0, 1, 2, 2, 3, 0 };
	
	MeshComponent->UpdateMesh(WorldTransform.GetLocation(), WorldTransform.GetRotation(), WorldTransform.GetScale3D(), Vertices, Indices);
}

void UARPlaneComponent::SetObjectClassificationDebugColors(const TMap<EARObjectClassification, FLinearColor>& InColors)
{
	ClassificationColors = InColors;
}

const TMap<EARObjectClassification, FLinearColor>& UARPlaneComponent::GetObjectClassificationDebugColors()
{
	return ClassificationColors;
}

void UARPlaneComponent::UpdateVisualization_Implementation()
{
	const auto& SessionPayload = ReplicatedPayload.SessionPayload;
	
	const bool bDebugModeEnabled = GetDebugMode() != 0;
	
	if (!MRMeshComponent)
	{
		MRMeshComponent = CreateMRMeshComponent(GetOwner(), SessionPayload, DefaultMeshMaterial, DefaultWireframeMeshMaterial, bDebugModeEnabled);
		OnMRMeshCreated.Broadcast(MRMeshComponent);
	}
	
	if (MRMeshComponent)
	{
		if (ReplicatedPayload.BoundaryVertices.Num())
		{
			auto Vertices = ReplicatedPayload.BoundaryVertices;
			const auto NumPolygons = Vertices.Num();
			TArray<MRMESH_INDEX_TYPE> Indices;
			Indices.Reset(3 * NumPolygons);
			for (auto Index = 0; Index < NumPolygons; ++Index)
			{
				Indices.Add(0);
				Indices.Add((Index + 1) % NumPolygons);
				Indices.Add((Index + 2) % NumPolygons);
			}
			
			MRMeshComponent->UpdateMesh(ReplicatedPayload.WorldTransform.GetLocation(), ReplicatedPayload.WorldTransform.GetRotation(), ReplicatedPayload.WorldTransform.GetScale3D(), Vertices, Indices);
		}
		else
		{
			UpdatePlaneMesh(MRMeshComponent, ReplicatedPayload.WorldTransform, ReplicatedPayload.Center, ReplicatedPayload.Extents);
		}
		
#if AR_DEBUG_MODE
		ManageMeshComponentForDebugMode(bDebugModeEnabled, SessionPayload);
		if (bDebugModeEnabled && MRMeshComponent)
		{
			if (GetDebugMode() == (int32)EPlaneComponentDebugMode::ShowNetworkRole)
			{
				MRMeshComponent->SetWireframeColor(GetDebugColor());
			}
			else if (GetDebugMode() == (int32)EPlaneComponentDebugMode::ShowClassification)
			{
				const auto ClassificationColor = ClassificationColors.Find(ReplicatedPayload.ObjectClassification);
				MRMeshComponent->SetWireframeColor(ClassificationColor ? *ClassificationColor : FLinearColor::White);
			}
		}
#endif
	}
}

void UARPlaneComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	const bool bDebugModeEnabled = GetDebugMode() != 0;
	if (bDebugModeEnabled)
	{
		FString DebugString;
		if (GetDebugMode() == (int32)EPlaneComponentDebugMode::ShowNetworkRole)
		{
			DebugString = MyTrackedGeometry ? TEXT("Local") : TEXT("Remote");
		}
		else if (GetDebugMode() == (int32)EPlaneComponentDebugMode::ShowClassification)
		{
			static const TMap<EARObjectClassification, FString> ClassificationNames =
			{
				{ EARObjectClassification::NotApplicable, TEXT("NotApplicable") },
				{ EARObjectClassification::Unknown, TEXT("Unknown") },
				{ EARObjectClassification::Wall, TEXT("Wall") },
				{ EARObjectClassification::Ceiling, TEXT("Ceiling") },
				{ EARObjectClassification::Floor, TEXT("Floor") },
				{ EARObjectClassification::Table, TEXT("Table") },
				{ EARObjectClassification::Seat, TEXT("Seat") },
				{ EARObjectClassification::Face, TEXT("Face") },
				{ EARObjectClassification::Image, TEXT("Image") },
				{ EARObjectClassification::World, TEXT("World") },
				{ EARObjectClassification::SceneObject, TEXT("SceneObject") },
				{ EARObjectClassification::HandMesh, TEXT("HandMesh") },
				{ EARObjectClassification::Door, TEXT("Door") },
				{ EARObjectClassification::Window, TEXT("Window") },
			};
			
			const auto ClassificationName = ClassificationNames.Find(ReplicatedPayload.ObjectClassification);
			DebugString = ClassificationName ? *ClassificationName : TEXT("Unknown");
		}
		
		const auto AxisLength = 15.f;
		const auto& WorldTransform = ReplicatedPayload.WorldTransform;
		const auto CenterLocation = WorldTransform.TransformPosition(ReplicatedPayload.Center);
		UKismetSystemLibrary::DrawDebugCoordinateSystem(this, CenterLocation, WorldTransform.Rotator(), AxisLength);
		
		const auto StringLocation = WorldTransform.TransformPosition(ReplicatedPayload.Center + FVector(0, 0, AxisLength + 1.f));
		UKismetSystemLibrary::DrawDebugString(this, StringLocation, DebugString, nullptr, FLinearColor::Yellow);
	}
#endif
}

void UARQRCodeComponent::UpdateVisualization_Implementation()
{
	const auto& SessionPayload = ReplicatedPayload.SessionPayload;
	
	const bool bDebugModeEnabled = GetDebugMode() != 0;
	
	if (!MRMeshComponent)
	{
		MRMeshComponent = CreateMRMeshComponent(GetOwner(), SessionPayload, DefaultMeshMaterial, DefaultWireframeMeshMaterial, bDebugModeEnabled);
		OnMRMeshCreated.Broadcast(MRMeshComponent);
	}
	
	if (MRMeshComponent)
	{
		UpdatePlaneMesh(MRMeshComponent, ReplicatedPayload.WorldTransform, FVector::ZeroVector, ReplicatedPayload.Extents);
		
#if AR_DEBUG_MODE
		ManageMeshComponentForDebugMode(bDebugModeEnabled, SessionPayload);
#endif
	}
}

void UARImageComponent::UpdateVisualization_Implementation()
{
	const auto& SessionPayload = ReplicatedPayload.SessionPayload;
	
	const bool bDebugModeEnabled = GetDebugMode() != 0;
	
	if (!MRMeshComponent)
	{
		MRMeshComponent = CreateMRMeshComponent(GetOwner(), SessionPayload, DefaultMeshMaterial, DefaultWireframeMeshMaterial, bDebugModeEnabled);
		OnMRMeshCreated.Broadcast(MRMeshComponent);
	}
	
	if (MRMeshComponent)
	{
		FVector Extent(ReplicatedPayload.EstimatedSize.X / 2.f, ReplicatedPayload.EstimatedSize.Y / 2.f, 0.f);
		UpdatePlaneMesh(MRMeshComponent, ReplicatedPayload.WorldTransform, FVector::ZeroVector, Extent);
		
#if AR_DEBUG_MODE
		ManageMeshComponentForDebugMode(bDebugModeEnabled, SessionPayload);
#endif
	}
}

void UARImageComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	const bool bDebugModeEnabled = GetDebugMode() != 0;
	if (bDebugModeEnabled)
	{
		FString DebugString;
		
		if (ReplicatedPayload.DetectedImage)
		{
			DebugString = FString::Printf(TEXT("%s (%s)"), *ReplicatedPayload.DetectedImage->GetName(), *ReplicatedPayload.DetectedImage->GetFriendlyName());
		}
		else
		{
			DebugString = TEXT("Unknown");
		}
		
		const auto AxisLength = 15.f;
		const auto& WorldTransform = ReplicatedPayload.WorldTransform;
		UKismetSystemLibrary::DrawDebugCoordinateSystem(this, WorldTransform.GetLocation(), WorldTransform.Rotator(), AxisLength);
		
		const auto StringLocation = WorldTransform.TransformPosition(FVector(0, 0, AxisLength + 1.f));
		UKismetSystemLibrary::DrawDebugString(this, StringLocation, DebugString, nullptr, FLinearColor::Yellow);
	}
#endif
}

void UARMeshComponent::UpdateVisualization_Implementation()
{
	const auto& SessionPayload = ReplicatedPayload.SessionPayload;
	// Currently there's not enough data in the payload to update the MRMeshComponent here
	// but we still need to create it so that the underlying XR system can update it internally
	if (!MRMeshComponent)
	{
		MRMeshComponent = CreateMRMeshComponent(GetOwner(), SessionPayload, DefaultMeshMaterial, DefaultWireframeMeshMaterial);
		OnMRMeshCreated.Broadcast(MRMeshComponent);
	}
}

template<typename VertexType>
void FAccumulatedNormal::CalculateVertexNormals(TArray<FAccumulatedNormal>& AccumulatedNormals, const TArray<VertexType>& Vertices, const TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FPackedNormal>& OutTangentData, FVector MeshCenter, float PositionScale)
{
	SCOPE_CYCLE_COUNTER(STAT_CalculateVertexNormals);
	
	const auto NumVertices = Vertices.Num();
	AccumulatedNormals.Reset(NumVertices);
	AccumulatedNormals.SetNumZeroed(NumVertices);
	// Manually going through each face of the mesh and calculate the face normal
	for (auto Index = 0; Index < Indices.Num(); Index += 3)
	{
		uint32 VertIndices[3] =
		{
			Indices[Index],
			Indices[Index + 1],
			Indices[Index + 2]
		};
		const FVector Verts[3] =
		{
			(FVector)Vertices[VertIndices[0]] * PositionScale,
			(FVector)Vertices[VertIndices[1]] * PositionScale,
			(FVector)Vertices[VertIndices[2]] * PositionScale
		};
		auto FaceNormal = (Verts[0] - Verts[1]) ^ (Verts[2] - Verts[0]);
		FaceNormal.Normalize();
		
		// Add the face normal to the accumulated normals for all 3 vertices
		for (auto VertIndex = 0; VertIndex < 3; ++VertIndex)
		{
			auto& AccumulatedNormal = AccumulatedNormals[VertIndices[VertIndex]];
			AccumulatedNormal.Normal = AccumulatedNormal.Normal + FaceNormal;
			++AccumulatedNormal.NumFaces;
		}
	}
	
	OutTangentData.SetNumUninitialized(NumVertices * 2);
	for (auto Index = 0; Index < NumVertices; ++Index)
	{
		// Calculate the average normal for each vertex
		auto& AccumulatedNormal = AccumulatedNormals[Index];
		if (ensure(AccumulatedNormal.NumFaces))
		{
			AccumulatedNormal.Normal /= (float)AccumulatedNormal.NumFaces;
			AccumulatedNormal.Normal.Normalize();
		}
		else
		{
			AccumulatedNormal.Normal = FVector(0, 0, 1);
		}
		
		// We don't have UV info to calculate the correct tangent here
		// so juse use an arbitrary vector to get a tangent perpendicular to the normal...
		const auto Cross = (MeshCenter - (FVector)Vertices[Index]) ^ AccumulatedNormal.Normal;
		auto Tangent = (AccumulatedNormal.Normal ^ Cross);
		Tangent.Normalize();
		OutTangentData[2 * Index + 0] = FPackedNormal(Tangent);
		OutTangentData[2 * Index + 1] = FPackedNormal(AccumulatedNormal.Normal);
	}
}
// instantiate for FVector3d and FVector3f
template void FAccumulatedNormal::CalculateVertexNormals<FVector3d>(TArray<FAccumulatedNormal>& AccumulatedNormals, const TArray<FVector3d>& Vertices, const TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FPackedNormal>& OutTangentData, FVector MeshCenter, float PositionScale);
template void FAccumulatedNormal::CalculateVertexNormals<FVector3f>(TArray<FAccumulatedNormal>& AccumulatedNormals, const TArray<FVector3f>& Vertices, const TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FPackedNormal>& OutTangentData, FVector MeshCenter, float PositionScale);

void UARFaceComponent::UpdateVisualization_Implementation()
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateFaceComponent);
	
	const auto& SessionPayload = ReplicatedPayload.SessionPayload;
	
	const bool bShowFaceMesh = GetDebugMode() == (int32)EFaceComponentDebugMode::ShowFaceMesh;
	
	if (!MRMeshComponent)
	{
		MRMeshComponent = CreateMRMeshComponent(GetOwner(), SessionPayload, DefaultMeshMaterial, DefaultWireframeMeshMaterial, bShowFaceMesh);
		OnMRMeshCreated.Broadcast(MRMeshComponent);
	}
	
	if (MRMeshComponent)
	{
		ManageMeshComponentForDebugMode(bShowFaceMesh, SessionPayload);
		if (MRMeshComponent)
		{
			if (auto FaceGeometry = Cast<UARFaceGeometry>(MyTrackedGeometry))
			{
				auto RenderTransform = FTransform::Identity;
				auto FaceTransform = FaceGeometry->GetLocalToWorldTransform();
				switch (TransformSetting)
				{
					case EARFaceTransformMixing::ComponentOnly:
					{
						RenderTransform = GetComponentTransform();
						break;
					}
						
					case EARFaceTransformMixing::ComponentLocationTrackedRotation:
					{
						RenderTransform = GetComponentTransform();
						FQuat TrackedRot = FaceTransform.GetRotation();
						RenderTransform.SetRotation(TrackedRot);
						break;
					}
						
					case EARFaceTransformMixing::ComponentWithTracked:
					{
						RenderTransform = GetComponentTransform() * FaceTransform;
						break;
					}
					case EARFaceTransformMixing::TrackingOnly:
					{
						RenderTransform = FaceTransform;
						break;
					}
				}

				auto bDefaultFaceOutOfScreen = true;
#if PLATFORM_IOS
				// Convert the vertices from ARKit's unit (meters) to UE4 units
				RenderTransform.MultiplyScale3D(FVector(100.f));
				// ARKit face into the screen by default
				bDefaultFaceOutOfScreen = false;
#endif
				
				if (bDefaultFaceOutOfScreen != bFaceOutOfScreen)
				{
					RenderTransform = FTransform(FRotator(0, 180, 0), FVector::ZeroVector) * RenderTransform;
				}
				
				auto Vertices = FaceGeometry->GetVertexBuffer();
				if (const auto NumVertices = Vertices.Num())
				{
					const auto& UV = FaceGeometry->GetUVs();
					const auto& IndexBuffer = FaceGeometry->GetIndexBuffer();
					
					const auto NumIndices = IndexBuffer.Num();
					TArray<MRMESH_INDEX_TYPE> Indices;
					Indices.AddUninitialized(NumIndices);
					for (auto Index = 0; Index < NumIndices; ++Index)
					{
						Indices[Index] = IndexBuffer[Index];
					}
					
					if (bUpdateVertexNormal)
					{
						FAccumulatedNormal::CalculateVertexNormals(AccumulatedNormals, Vertices, Indices, TangentData, FVector::ZeroVector, 100.f);
					}
					
					MRMeshComponent->UpdateMesh(RenderTransform.GetLocation(), RenderTransform.GetRotation(), RenderTransform.GetScale3D(), Vertices, Indices, UV, bUpdateVertexNormal ? TangentData : TArray<FPackedNormal>(), {});
				}
			}
		}
	}
}

void UARFaceComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	if (GetDebugMode() == (int32)EFaceComponentDebugMode::ShowEyeVectors)
	{
		UKismetSystemLibrary::DrawDebugLine(this, ReplicatedPayload.LookAtTarget, ReplicatedPayload.LeftEyePosition, GetDebugColor());
		UKismetSystemLibrary::DrawDebugLine(this, ReplicatedPayload.LookAtTarget, ReplicatedPayload.RightEyePosition, GetDebugColor());
	}
#endif
}

void UARPoseComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	if (GetDebugMode() == (int32)EPoseComponentDebugMode::ShowSkeleton)
	{
		for (const auto& Joint : ReplicatedPayload.JointTransforms)
		{
			UKismetSystemLibrary::DrawDebugPoint(this, ReplicatedPayload.WorldTransform.TransformPosition(Joint.GetLocation()), 2.f, GetDebugColor());
		}
	}
#endif
}

void UARQRCodeComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	if (GetDebugMode() == (int32)EQRCodeComponentDebugMode::ShowQRCode)
	{
		const auto& WorldTransform = ReplicatedPayload.WorldTransform;
		const auto StringLocation = WorldTransform.TransformPosition(FVector(0, 0, 10.f));
		UKismetSystemLibrary::DrawDebugString(this, StringLocation, ReplicatedPayload.QRCode, nullptr, FLinearColor::Yellow);
	}
#endif
}

void UARGeoAnchorComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	
#if AR_DEBUG_MODE
	if (GetDebugMode() == (int32)EGeoAnchorComponentDebugMode::ShowGeoData)
	{
		const auto& WorldTransform = ReplicatedPayload.WorldTransform;
		const auto StringLocation = WorldTransform.TransformPosition(FVector(0, 0, 10.f));
		
		static const TMap<EARAltitudeSource, FString> AltitudeSourceNames =
		{
			{ EARAltitudeSource::Precise,	 	TEXT("Precise") },
			{ EARAltitudeSource::Coarse, 		TEXT("Coarse") },
			{ EARAltitudeSource::UserDefined, 	TEXT("UserDefined") },
			{ EARAltitudeSource::Unknown, 		TEXT("Unknown") },
		};
		
		const auto DebugString = FString::Printf(TEXT("%s: Longitude (%f), Latitude (%f), AltitudeMeters (%f), AltitudeSource (%s)"),
												 ReplicatedPayload.AnchorName.Len() ? *ReplicatedPayload.AnchorName : TEXT("Geo Anchor"),
												 ReplicatedPayload.Longitude, ReplicatedPayload.Latitude, ReplicatedPayload.AltitudeMeters,
												 *AltitudeSourceNames[ReplicatedPayload.AltitudeSource]);
		UKismetSystemLibrary::DrawDebugString(this, StringLocation, DebugString, nullptr, FLinearColor::Yellow);
		
		const auto AxisLength = 15.f;
		UKismetSystemLibrary::DrawDebugCoordinateSystem(this, WorldTransform.GetLocation(), WorldTransform.Rotator(), AxisLength);
	}
#endif
}

