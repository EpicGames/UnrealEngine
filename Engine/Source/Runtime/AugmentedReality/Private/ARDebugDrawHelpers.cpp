// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARDebugDrawHelpers.h"


#include "DrawDebugHelpers.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "GameFramework/HUD.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/PlayerController.h"

namespace ARDebugHelpers
{
	void DrawDebugString(const UWorld* InWorld, FVector const& TextLocation, const FString& Text, float Scale, FColor const& TextColor, float Duration, bool bDrawShadow)
	{
		bool bDrawnSuccessfully = false;
		// no debug line drawing on dedicated server
		if (GEngine->GetNetMode(InWorld) != NM_DedicatedServer)
		{
			check(InWorld);
			AActor* BaseAct = InWorld->GetWorldSettings();
			
			// iterate through the player controller list
			for( FConstPlayerControllerIterator Iterator = InWorld->GetPlayerControllerIterator(); Iterator; ++Iterator )
			{
				APlayerController* PlayerController = Iterator->Get();
				if (PlayerController && PlayerController->MyHUD && PlayerController->Player)
				{
					PlayerController->MyHUD->AddDebugText(Text, BaseAct, Duration, TextLocation, TextLocation, TextColor, true, true, false, nullptr, Scale, bDrawShadow);
					bDrawnSuccessfully = true;
				}
			}
		}
		
		ensureMsgf(bDrawnSuccessfully, TEXT("You must have am AHUD actor assigned for DrawDebugString to work."));
	}
}
