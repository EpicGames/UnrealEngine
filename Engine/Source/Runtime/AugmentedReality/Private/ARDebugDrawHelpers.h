// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class UWorld;
class FString;
struct FColor;

namespace ARDebugHelpers
{
	void DrawDebugString(const UWorld* InWorld, FVector const& TextLocation, const FString& Text, float Scale, FColor const& TextColor, float Duration, bool bDrawShadow);
}
