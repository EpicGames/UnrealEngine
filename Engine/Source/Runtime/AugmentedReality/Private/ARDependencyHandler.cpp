// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARDependencyHandler.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARDependencyHandler)


UARDependencyHandler* UARDependencyHandler::GetARDependencyHandler()
{
	auto& ModularFeatures = IModularFeatures::Get();
	const auto FeatureName = GetModularFeatureName();
	if (ModularFeatures.GetModularFeatureImplementationCount(FeatureName))
	{
		return static_cast<UARDependencyHandler*>(ModularFeatures.GetModularFeatureImplementation(FeatureName, 0));
	}
	return nullptr;
}

