// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARGeoTrackingSupport.h"
#include "ARBlueprintLibrary.h"
#include "Features/IModularFeatures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARGeoTrackingSupport)


UARGeoTrackingSupport* UARGeoTrackingSupport::GetGeoTrackingSupport()
{
	auto& ModularFeatures = IModularFeatures::Get();
	const auto FeatureName = GetModularFeatureName();
	if (ModularFeatures.GetModularFeatureImplementationCount(FeatureName))
	{
		return static_cast<UARGeoTrackingSupport*>(ModularFeatures.GetModularFeatureImplementation(FeatureName, 0));
	}
	return nullptr;
}

TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> UARGeoTrackingSupport::CheckGeoTrackingAvailability(FString& OutError)
{
	OutError = TEXT("Not implemented");
	return nullptr;
}

TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> UARGeoTrackingSupport::CheckGeoTrackingAvailability(float Longitude, float Latitude, FString& OutError)
{
	OutError = TEXT("Not implemented");
	return nullptr;
}

TSharedPtr<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe> UARGeoTrackingSupport::GetGeoLocationAtWorldPosition(const FVector& WorldPosition, FString& OutError)
{
	OutError = TEXT("Not implemented");
	return nullptr;
}

void FCheckGeoTrackingAvailabilityAsyncTask::FinishWithAvailability(bool bInIsAvailable)
{
	bIsAvailable = bInIsAvailable;
	bIsDone = true;
	bHadError = false;
}

void FCheckGeoTrackingAvailabilityAsyncTask::FinishWithError(const FString& InError)
{
	bIsAvailable = false;
	bIsDone = true;
	bHadError = true;
	Error = InError;
}

UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy* UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy::CheckGeoTrackingAvailability(UObject* WorldContextObject)
{
	auto Task = NewObject<UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy>();
	Task->RegisterWithGameInstance(WorldContextObject);
	return Task;
}

UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy* UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy::CheckGeoTrackingAvailabilityAtLocation(UObject* WorldContextObject, float Longitude, float Latitude)
{
	auto Task = NewObject<UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy>();
	Task->RegisterWithGameInstance(WorldContextObject);
	Task->Longitude = Longitude;
	Task->Latitude = Latitude;
	return Task;
}

void UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy::Activate()
{
	if (auto GeoTrackingSupport = UARGeoTrackingSupport::GetGeoTrackingSupport())
	{
		if (Longitude.IsSet() && Latitude.IsSet())
		{
			MyTask = GeoTrackingSupport->CheckGeoTrackingAvailability(*Longitude, *Latitude, Error);
		}
		else
		{
			MyTask = GeoTrackingSupport->CheckGeoTrackingAvailability(Error);
		}
		
		if (MyTask)
		{
			AsyncTask = MyTask;
		}
	}
}

void UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy::ReportSuccess()
{
	check(MyTask);
	OnSuccess.Broadcast(MyTask->IsAvailable(), {});
}

void UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy::ReportFailure()
{
	if (MyTask)
	{
		OnFailed.Broadcast(false, MyTask->GetErrorString());
	}
	else
	{
		OnFailed.Broadcast(false, Error);
	}
}

void FGetGeoLocationAsyncTask::FinishWithGeoLocation(float InLongitude, float InLatitude, float InAltitude)
{
	Longitude = InLongitude;
	Latitude = InLatitude;
	Altitude = InAltitude;
	bIsDone = true;
	bHadError = false;
}

void FGetGeoLocationAsyncTask::FinishWithError(const FString& InError)
{
	bIsDone = true;
	bHadError = true;
	Error = InError;
}

UGetGeoLocationAsyncTaskBlueprintProxy* UGetGeoLocationAsyncTaskBlueprintProxy::GetGeoLocationAtWorldPosition(UObject* WorldContextObject, const FVector& WorldPosition)
{
	auto Task = NewObject<UGetGeoLocationAsyncTaskBlueprintProxy>();
	Task->RegisterWithGameInstance(WorldContextObject);
	Task->WorldPosition = WorldPosition;
	return Task;
}

void UGetGeoLocationAsyncTaskBlueprintProxy::Activate()
{
	if (auto GeoTrackingSupport = UARGeoTrackingSupport::GetGeoTrackingSupport())
	{
		MyTask = GeoTrackingSupport->GetGeoLocationAtWorldPosition(WorldPosition, Error);
		if (MyTask)
		{
			AsyncTask = MyTask;
		}
	}
}

void UGetGeoLocationAsyncTaskBlueprintProxy::ReportSuccess()
{
	check(MyTask);
	OnSuccess.Broadcast(MyTask->GetLongitude(), MyTask->GetLatitude(), MyTask->GetAltitude(), {});
}

void UGetGeoLocationAsyncTaskBlueprintProxy::ReportFailure()
{
	if (MyTask)
	{
		OnFailed.Broadcast(0, 0, 0, MyTask->GetErrorString());
	}
	else
	{
		OnFailed.Broadcast(0, 0, 0, Error);
	}
}

