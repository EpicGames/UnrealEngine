// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARLifeCycleComponent.h"
#include "AugmentedRealityModule.h"
#include "ARActor.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "GameFramework/PlayerController.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARLifeCycleComponent)


UARLifeCycleComponent::FRequestSpawnARActorDelegate UARLifeCycleComponent::RequestSpawnARActorDelegate;
UARLifeCycleComponent::FOnSpawnARActorDelegate UARLifeCycleComponent::OnSpawnARActorDelegate;
UARLifeCycleComponent::FRequestDestroyARActorDelegate UARLifeCycleComponent::RequestDestroyARActorDelegate;

void UARLifeCycleComponent::OnComponentCreated()
{
	Super::OnComponentCreated();

	if (!SpawnDelegateHandle.IsValid())
	{
		APlayerController* PC = Cast<APlayerController>(GetOwner());
		if (PC)
		{
			if (PC->IsLocalController())
			{
				//the server should not have these bound.  Only clients which can then use BPs to send them to the server
				SpawnDelegateHandle = RequestSpawnARActorDelegate.AddUObject(this, &UARLifeCycleComponent::CallInstanceRequestSpawnARActorDelegate);
				DestroyDelegateHandle = RequestDestroyARActorDelegate.AddUObject(this, &UARLifeCycleComponent::CallInstanceRequestDestroyARActorDelegate);
			}
		}
		else
		{
			UE_LOGF(LogAR, Error, "UARLifeCycleComponent should be attached to a player controller");
		}
	}
}

void UARLifeCycleComponent::DestroyComponent(bool bPromoteChildren)
{
	Super::DestroyComponent(bPromoteChildren);

	RequestSpawnARActorDelegate.Remove(SpawnDelegateHandle);
	RequestDestroyARActorDelegate.Remove(DestroyDelegateHandle);
}

void UARLifeCycleComponent::OnUnregister()
{
	if (auto MyOwner = GetOwner())
	{
		if (MyOwner->HasAuthority())
		{
			// Collect and destroy all the AR actors that share the same owner as this component
			TArray<AARActor*> ActorsToDestroy;
			for (TActorIterator<AARActor> ActorItr(GetWorld()); ActorItr; ++ActorItr)
			{
				AARActor* ARActor = *ActorItr;
				if (ARActor->GetOwner() == MyOwner)
				{
					ActorsToDestroy.Add(ARActor);
				}
			}
			
			for (auto ARActor : ActorsToDestroy)
			{
				UE_LOGF(LogAR, Log, "Destroying AR actor on the server as part of cleanup: [%ls]", *ARActor->GetName());
				ARActor->Destroy();
			}
		}
	}
	
	Super::OnUnregister();
}

void UARLifeCycleComponent::CallInstanceRequestSpawnARActorDelegate(UClass* Class, FGuid NativeID)
{
	ServerSpawnARActor(Class, NativeID);
}

void UARLifeCycleComponent::CallInstanceRequestDestroyARActorDelegate(AARActor* Actor)
{
	ServerDestroyARActor(Actor);
}

bool UARLifeCycleComponent::ServerSpawnARActor_Validate(UClass* ComponentClass, FGuid NativeID)
{
	return true;
}

void UARLifeCycleComponent::ServerSpawnARActor_Implementation(UClass* ComponentClass, FGuid NativeID)
{
	if (ComponentClass)
	{
		const auto SpawnTransform = FTransform::Identity;
		if (auto ARActor = Cast<AARActor>(GetWorld()->SpawnActor(AARActor::StaticClass(), &SpawnTransform)))
		{
			ARActor->SetOwner(GetOwner());
			ARActor->AddARComponent(ComponentClass, NativeID);
			UE_LOGF(LogAR, Log, "Spawned AR actor on the server: [%ls] with component class [%ls] and GUID [%ls]",
				   *ARActor->GetName(), *ComponentClass->GetName(), *NativeID.ToString());
			
			OnARActorSpawnedDelegate.Broadcast(ComponentClass, NativeID, ARActor);
		}
	}
}

bool UARLifeCycleComponent::ServerDestroyARActor_Validate(AARActor* Actor)
{
	return true;
}

void UARLifeCycleComponent::ServerDestroyARActor_Implementation(AARActor* Actor)
{
	if (Actor)
	{
		UE_LOGF(LogAR, Log, "Destroying AR actor on the server: [%ls]", *Actor->GetName());
		OnARActorToBeDestroyedDelegate.Broadcast(Actor);
		Actor->Destroy();
	}
}

