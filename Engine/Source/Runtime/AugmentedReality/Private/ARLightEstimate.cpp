// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARLightEstimate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARLightEstimate)

//
//
//
void UARBasicLightEstimate::SetLightEstimate(float InAmbientIntensityLumens, float InColorTemperatureKelvin)
{
	AmbientIntensityLumens = InAmbientIntensityLumens;
	AmbientColorTemperatureKelvin = InColorTemperatureKelvin;
	AmbientColor = 	FLinearColor::MakeFromColorTemperature(GetAmbientColorTemperatureKelvin());
}

void UARBasicLightEstimate::SetLightEstimate(FVector InRGBScaleFactor, float InPixelIntensity)
{
	// Try to convert ARCore average pixel intensity to lumen and set the color tempature to pure white.
	AmbientIntensityLumens = InPixelIntensity / 0.18f * 1000;
	AmbientColor = FLinearColor(InRGBScaleFactor);
	
	// TODO: Try to convert ambient color to color tempature?
}

void UARBasicLightEstimate::SetLightEstimate(float InColorTemperatureKelvin, FLinearColor InAmbientColor)
{
	AmbientColorTemperatureKelvin = InColorTemperatureKelvin;
	AmbientColor = InAmbientColor;
}

float UARBasicLightEstimate::GetAmbientIntensityLumens() const
{
	return AmbientIntensityLumens;
}

float UARBasicLightEstimate::GetAmbientColorTemperatureKelvin() const
{
	return AmbientColorTemperatureKelvin;
}

FLinearColor UARBasicLightEstimate::GetAmbientColor() const
{
	return AmbientColor;
}

