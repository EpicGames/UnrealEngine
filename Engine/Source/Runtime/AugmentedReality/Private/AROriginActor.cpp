// Copyright Epic Games, Inc. All Rights Reserved.

#include "AROriginActor.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AROriginActor)

AAROriginActor::AAROriginActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = false;
	bAllowTickBeforeBeginPlay = false;
	bReplicates = false;
	SetReplicatingMovement(false);
	SetCanBeDamaged(false);
}

AAROriginActor* AAROriginActor::GetOriginActor()
{
	// Have to find the game world, not the editor world, if we are in vr preview
	UWorld* GameWorld = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Game || Context.WorldType == EWorldType::PIE)
		{
			GameWorld = Context.World();
		}
	}

	if (GameWorld != nullptr)
	{
		AAROriginActor* FoundActor = nullptr;
		for (TActorIterator<AAROriginActor> Iter(GameWorld); Iter; ++Iter)
		{
			if (IsValid(*Iter))
			{
				FoundActor = *Iter;
				break;
			}
		}
		if (FoundActor == nullptr)
		{
			// None spawned yet
			FoundActor = GameWorld->SpawnActor<AAROriginActor>(AAROriginActor::StaticClass(), FVector::ZeroVector, FRotator::ZeroRotator);
		}
		return FoundActor;
	}
	return nullptr;
}

