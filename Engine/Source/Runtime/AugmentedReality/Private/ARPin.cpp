// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARPin.h"
#include "ARSupportInterface.h"
#include "ARSystem.h"
#include "ARDebugDrawHelpers.h"
#include "DrawDebugHelpers.h"
#include "Components/SceneComponent.h"
#include "IXRTrackingSystem.h"

#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARPin)

//
//
//

uint32 UARPin::DebugPinId = 0;

void UARPin::InitARPin( const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystemOwner, USceneComponent* InComponentToPin, const FTransform& InLocalToTrackingTransform, UARTrackedGeometry* InTrackedGeometry, const FName InDebugName )
{
	ARSystem = InTrackingSystemOwner;
	LocalToTrackingTransform = InLocalToTrackingTransform;
	LocalToAlignedTrackingTransform = LocalToTrackingTransform * InTrackingSystemOwner->GetAlignmentTransform();
	TrackingState = EARTrackingState::Tracking;
	TrackedGeometry = InTrackedGeometry;
	PinnedComponent = InComponentToPin;

	// Move the component to match the Pin's new location
	if (PinnedComponent)
	{
		const FTransform NewLocalToWorldTransform = GetLocalToWorldTransform();
		PinnedComponent->SetWorldTransform(NewLocalToWorldTransform);
	}

	DebugName = (InDebugName != NAME_None)
	? InDebugName
	: FName(*FString::Printf(TEXT("PIN %02d"), DebugPinId++));
}

FTransform UARPin::GetLocalToTrackingTransform() const
{
	return LocalToAlignedTrackingTransform;
}

FTransform UARPin::GetLocalToWorldTransform() const
{
	return GetLocalToTrackingTransform() * GetARSystem()->GetXRTrackingSystem()->GetTrackingToWorldTransform();
}

EARTrackingState UARPin::GetTrackingState() const
{
	return TrackingState;
}

UARTrackedGeometry* UARPin::GetTrackedGeometry() const
{
	return TrackedGeometry;
}

USceneComponent* UARPin::GetPinnedComponent() const
{
	return PinnedComponent;
}


FTransform UARPin::GetLocalToTrackingTransform_NoAlignment() const
{
	return LocalToTrackingTransform;
}


void UARPin::OnTrackingStateChanged(EARTrackingState NewTrackingState)
{
	if (NewTrackingState == EARTrackingState::StoppedTracking)
	{
		TrackedGeometry = nullptr;
		DebugName = FName( *FString::Printf(TEXT("%s [ORPHAN]"), *DebugName.ToString()) );
	}
	
	TrackingState = NewTrackingState;
	OnARTrackingStateChanged.Broadcast(NewTrackingState);
}


void UARPin::OnTransformUpdated(const FTransform& NewLocalToTrackingTransform)
{
	FTransform LocalToNewLocal = LocalToTrackingTransform.GetRelativeTransform(NewLocalToTrackingTransform);
	
	LocalToTrackingTransform = NewLocalToTrackingTransform;
	LocalToAlignedTrackingTransform = LocalToTrackingTransform * GetARSystem()->GetAlignmentTransform();
	
	// Move the component to match the Pin's new location
	if (PinnedComponent)
	{
		const FTransform NewLocalToWorldTransform = GetLocalToWorldTransform();
		PinnedComponent->SetWorldTransform(NewLocalToWorldTransform);
	}

	// Notify any subscribes that the Pin moved
	OnARTransformUpdated.Broadcast(LocalToNewLocal);
}

void UARPin::UpdateAlignmentTransform( const FTransform& NewAlignmentTransform )
{
	const FTransform OldLocalToAlignedTracking = LocalToAlignedTrackingTransform;
	
	LocalToAlignedTrackingTransform = LocalToTrackingTransform * NewAlignmentTransform;
	
	// Note that when the alignment transform changes, we are not going to rely on geometries sending updates.
	// Instead, we are going to broadcast an update that directly modifies all the geometries and pins.
	// Move the component to match the Pin's new location
	if (PinnedComponent)
	{
		const FTransform NewLocalToWorldTransform = GetLocalToWorldTransform();
		PinnedComponent->SetWorldTransform(NewLocalToWorldTransform);
	}
	
	FTransform DeltaTransform = OldLocalToAlignedTracking.GetRelativeTransform(LocalToAlignedTrackingTransform);

	OnARTransformUpdated.Broadcast( DeltaTransform );
}

void UARPin::DebugDraw( UWorld* World, const FLinearColor& Color, float Scale, float PersistForSeconds) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FVector Location_WorldSpace = LocalToWorldTransform.GetLocation();
	DrawDebugCrosshairs( World, Location_WorldSpace, FRotator(LocalToWorldTransform.GetRotation()), Scale, Color.ToFColor(false), (PersistForSeconds > 0.0f), PersistForSeconds );
	
	const FName MyDebugName = GetDebugName();
	const FString CurAnchorDebugName = MyDebugName.ToString();
	ARDebugHelpers::DrawDebugString( World, Location_WorldSpace, CurAnchorDebugName, 0.25f*Scale, Color.ToFColor(false), PersistForSeconds, true);
}

FName UARPin::GetDebugName() const
{
	return DebugName;
}

void UARPin::SetOnARTrackingStateChanged( const FOnARTrackingStateChanged& InHandler )
{
	OnARTrackingStateChanged = InHandler;
}

void UARPin::SetOnARTransformUpdated( const FOnARTransformUpdated& InHandler )
{
	OnARTransformUpdated = InHandler;
}

TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> UARPin::GetARSystem() const
{
	auto MyARSystem = ARSystem.Pin();
	return MyARSystem;
}

