// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSessionConfig.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/VRObjectVersion.h"
#include "Containers/StringConv.h"
#include "EngineLogs.h"
#include "Misc/CoreMisc.h"
#include "ARSessionConfigCookSupport.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARSessionConfig)

UARSessionConfig::UARSessionConfig()
	: bTrackSceneObjects(true)
	, WorldAlignment(EARWorldAlignment::Gravity)
	, SessionType(EARSessionType::World)
	, PlaneDetectionMode_DEPRECATED(EARPlaneDetectionMode::HorizontalPlaneDetection)
	, bHorizontalPlaneDetection(true)
	, bVerticalPlaneDetection(true)
	, bEnableAutoFocus(true)
	, LightEstimationMode(EARLightEstimationMode::AmbientLightEstimate)
	, FrameSyncMode(EARFrameSyncMode::SyncTickWithCameraImage)
	, bEnableAutomaticCameraOverlay(true)
	, bEnableAutomaticCameraTracking(true)
	, bResetCameraTracking(true)
	, bResetTrackedObjects(true)
	, MaxNumSimultaneousImagesTracked(1)
	, PlaneComponentClass(UARPlaneComponent::StaticClass())
	, PointComponentClass(UARPointComponent::StaticClass())
	, FaceComponentClass(UARFaceComponent::StaticClass())
	, ImageComponentClass(UARImageComponent::StaticClass())
	, QRCodeComponentClass(UARTrackedQRCode::StaticClass())
	, PoseComponentClass(UARPoseComponent::StaticClass())
	, EnvironmentProbeComponentClass(UAREnvironmentProbeComponent::StaticClass())
	, ObjectComponentClass(UARObjectComponent::StaticClass())
	, MeshComponentClass(UARMeshComponent::StaticClass())
	, GeoAnchorComponentClass(UARGeoAnchorComponent::StaticClass())
{
	DefaultMeshMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	DefaultWireframeMeshMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
}

EARWorldAlignment UARSessionConfig::GetWorldAlignment() const
{
	return WorldAlignment;
}

EARSessionType UARSessionConfig::GetSessionType() const
{
	return SessionType;
}

EARPlaneDetectionMode UARSessionConfig::GetPlaneDetectionMode() const
{
	return static_cast<EARPlaneDetectionMode>(
	(bHorizontalPlaneDetection ? static_cast<int32>(EARPlaneDetectionMode::HorizontalPlaneDetection) : 0) |
	(bVerticalPlaneDetection ? static_cast<int32>(EARPlaneDetectionMode::VerticalPlaneDetection) : 0));
}

EARLightEstimationMode UARSessionConfig::GetLightEstimationMode() const
{
	return LightEstimationMode;
}

EARFrameSyncMode UARSessionConfig::GetFrameSyncMode() const
{
	return FrameSyncMode;
}

bool UARSessionConfig::ShouldRenderCameraOverlay() const
{
	return bEnableAutomaticCameraOverlay;
}

bool UARSessionConfig::ShouldEnableCameraTracking() const
{
	return bEnableAutomaticCameraTracking;
}

bool UARSessionConfig::ShouldEnableAutoFocus() const
{
	return bEnableAutoFocus;
}

void UARSessionConfig::SetEnableAutoFocus(bool bNewValue)
{
	bEnableAutoFocus = bNewValue;
}

bool UARSessionConfig::ShouldResetCameraTracking() const
{
	return bResetCameraTracking;
}

void UARSessionConfig::SetResetCameraTracking(bool bNewValue)
{
	bResetCameraTracking = bNewValue;
}

bool UARSessionConfig::ShouldResetTrackedObjects() const
{
	return bResetTrackedObjects;
}

void UARSessionConfig::SetResetTrackedObjects(bool bNewValue)
{
	bResetTrackedObjects = bNewValue;
}

const TArray<UARCandidateImage*>& UARSessionConfig::GetCandidateImageList() const
{
	return CandidateImages;
}

void UARSessionConfig::AddCandidateImage(UARCandidateImage* NewCandidateImage)
{
	CandidateImages.Add(NewCandidateImage);
}

void UARSessionConfig::RemoveCandidateImage(UARCandidateImage* CandidateImage)
{
	int ImagesRemoved = CandidateImages.Remove(CandidateImage);
}

void UARSessionConfig::RemoveCandidateImageAtIndex(int Index)
{
	if (Index < 0 || Index >= CandidateImages.Num())
	{
		UE_LOGF(LogBlueprint, Warning, "RemoveCandidateImageAtIndex failed because the index is invalid.  No image removed.");
	}
	else
	{
		CandidateImages.RemoveAt(Index);
	}
}

void UARSessionConfig::ClearCandidateImages()
{
	CandidateImages.Empty();
}

int32 UARSessionConfig::GetMaxNumSimultaneousImagesTracked() const
{
    return MaxNumSimultaneousImagesTracked;
}

EAREnvironmentCaptureProbeType UARSessionConfig::GetEnvironmentCaptureProbeType() const
{
	return EnvironmentCaptureProbeType;
}

const TArray<uint8>& UARSessionConfig::GetWorldMapData() const
{
	return WorldMapData;
}

void UARSessionConfig::SetWorldMapData(TArray<uint8> InWorldMapData)
{
	WorldMapData = MoveTemp(InWorldMapData);
}

const TArray<UARCandidateObject*>& UARSessionConfig::GetCandidateObjectList() const
{
	return CandidateObjects;
}

void UARSessionConfig::SetCandidateObjectList(const TArray<UARCandidateObject*>& InCandidateObjects)
{
	CandidateObjects = InCandidateObjects;
}

void UARSessionConfig::AddCandidateObject(UARCandidateObject* CandidateObject)
{
	if (CandidateObject != nullptr)
	{
		CandidateObjects.Add(CandidateObject);
	}
}

const TArray<uint8>& UARSessionConfig::GetSerializedARCandidateImageDatabase() const
{
	return SerializedARCandidateImageDatabase;
}

UClass* UARSessionConfig::GetPlaneComponentClass(void) const
{
	return PlaneComponentClass.Get() ? PlaneComponentClass.Get() : UARPlaneComponent::StaticClass();
}

UClass* UARSessionConfig::GetPointComponentClass(void) const
{
	return PointComponentClass.Get() ? PointComponentClass.Get() : UARPointComponent::StaticClass();
}
UClass* UARSessionConfig::GetFaceComponentClass(void) const
{
	return FaceComponentClass.Get() ? FaceComponentClass.Get() : UARFaceComponent::StaticClass();
}
UClass* UARSessionConfig::GetImageComponentClass(void) const
{
	return ImageComponentClass.Get() ? ImageComponentClass.Get() : UARImageComponent::StaticClass();
}
UClass* UARSessionConfig::GetQRCodeComponentClass(void) const
{
	return QRCodeComponentClass.Get() ? QRCodeComponentClass.Get() : UARQRCodeComponent::StaticClass();
}
UClass* UARSessionConfig::GetPoseComponentClass(void) const
{
	return PoseComponentClass.Get() ? PoseComponentClass.Get() : UARPoseComponent::StaticClass();
}
UClass* UARSessionConfig::GetEnvironmentProbeComponentClass(void) const
{
	return EnvironmentProbeComponentClass.Get() ? EnvironmentProbeComponentClass.Get() : UAREnvironmentProbeComponent::StaticClass();
}
UClass* UARSessionConfig::GetObjectComponentClass(void) const
{
	return ObjectComponentClass.Get() ? ObjectComponentClass.Get() : UARObjectComponent::StaticClass();
}

UClass* UARSessionConfig::GetMeshComponentClass(void) const
{
	return MeshComponentClass.Get() ? MeshComponentClass.Get() : UARMeshComponent::StaticClass();
}

UClass* UARSessionConfig::GetGeoAnchorComponentClass(void) const
{
	return GeoAnchorComponentClass.Get() ? GeoAnchorComponentClass.Get() : UARGeoAnchorComponent::StaticClass();
}

FARVideoFormat UARSessionConfig::GetDesiredVideoFormat() const
{
	return DesiredVideoFormat;
}

void UARSessionConfig::SetDesiredVideoFormat(FARVideoFormat NewFormat)
{
	DesiredVideoFormat = NewFormat;
}

EARFaceTrackingDirection UARSessionConfig::GetFaceTrackingDirection() const
{
	return FaceTrackingDirection;
}

void UARSessionConfig::SetFaceTrackingDirection(EARFaceTrackingDirection InDirection)
{
	FaceTrackingDirection = InDirection;
}

EARFaceTrackingUpdate UARSessionConfig::GetFaceTrackingUpdate() const
{
	return FaceTrackingUpdate;
}

void UARSessionConfig::SetFaceTrackingUpdate(EARFaceTrackingUpdate InUpdate)
{
	FaceTrackingUpdate = InUpdate;
}

void UARSessionConfig::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FVRObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsLoading() && Ar.IsCooking())
	{
		TArray<IARSessionConfigCookSupport*> CookSupportModules = IModularFeatures::Get().GetModularFeatureImplementations<IARSessionConfigCookSupport>(IARSessionConfigCookSupport::GetModularFeatureName());
		for (IARSessionConfigCookSupport* CookSupportModule : CookSupportModules)
		{
			CookSupportModule->OnSerializeSessionConfig(this, Ar, SerializedARCandidateImageDatabase);
		}
	}
#endif

	Super::Serialize(Ar);

	if (Ar.CustomVer(FVRObjectVersion::GUID) < FVRObjectVersion::UseBoolsForARSessionConfigPlaneDetectionConfiguration)
	{
		if (PlaneDetectionMode_DEPRECATED == EARPlaneDetectionMode::None)
		{
			bHorizontalPlaneDetection = false;
			bVerticalPlaneDetection = false;
		}
	}
}

EARSessionTrackingFeature UARSessionConfig::GetEnabledSessionTrackingFeature() const
{
	return EnabledSessionTrackingFeature;
}

void UARSessionConfig::SetSessionTrackingFeatureToEnable(EARSessionTrackingFeature InSessionTrackingFeature)
{
	EnabledSessionTrackingFeature = InSessionTrackingFeature;
}

EARSceneReconstruction UARSessionConfig::GetSceneReconstructionMethod() const
{
	return SceneReconstructionMethod;
}

void UARSessionConfig::SetSceneReconstructionMethod(EARSceneReconstruction InSceneReconstructionMethod)
{
	SceneReconstructionMethod = InSceneReconstructionMethod;
}

bool UARSessionConfig::ShouldUseOptimalVideoFormat() const
{
	return bUseOptimalVideoFormat;
}

