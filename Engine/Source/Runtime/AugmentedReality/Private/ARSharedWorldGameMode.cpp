// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSharedWorldGameMode.h"
#include "Engine/World.h"
#include "AugmentedRealityModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARSharedWorldGameMode)

AARSharedWorldGameMode::AARSharedWorldGameMode(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, BufferSizePerChunk(512)
{
	GameStateClass = AARSharedWorldGameState::StaticClass();
	PlayerControllerClass = AARSharedWorldPlayerController::StaticClass();
}

void AARSharedWorldGameMode::SetPreviewImageData(TArray<uint8> ImageData)
{
	GetARSharedWorldGameState()->PreviewImageData = MoveTemp(ImageData);
}

void AARSharedWorldGameMode::SetARSharedWorldData(TArray<uint8> ARWorldData)
{
	GetARSharedWorldGameState()->ARWorldData = MoveTemp(ARWorldData);
}

AARSharedWorldGameState* AARSharedWorldGameMode::GetARSharedWorldGameState()
{
	return CastChecked<AARSharedWorldGameState>(GameState);
}

void AARSharedWorldGameMode::SetARWorldSharingIsReady()
{
	// We should only send the world once per game session
	if (!bShouldSendSharedWorldData)
	{
		bShouldSendSharedWorldData = true;

		const AARSharedWorldGameState* SharedGameState = GetARSharedWorldGameState();
		UE_LOGF(LogAR, Log, "Ready to share AR data with clients. AR world size is (%d) and preview image size is (%d)", SharedGameState->ARWorldData.Num(), SharedGameState->PreviewImageData.Num());
	}
}

void AARSharedWorldGameMode::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);
	
	if (bShouldSendSharedWorldData)
	{
		const AARSharedWorldGameState* SharedGameState = GetARSharedWorldGameState();

		// For each player, send them their next chunk of data if needed
		for (auto It = GetWorld()->GetControllerIterator(); It; ++It)
		{
			AARSharedWorldPlayerController* PC = Cast<AARSharedWorldPlayerController>(It->Get());
			if (PC != nullptr && !PC->IsLocalController() && PC->IsReadyToReceive())
			{
				if (PlayerToReplicationStateMap.Contains(PC))
				{
					FARSharedWorldReplicationState& ReplState = PlayerToReplicationStateMap[PC];
					// See if we need to send any image preview data
					int32 CurrentOffset = ReplState.PreviewImageOffset;
					if (CurrentOffset < SharedGameState->PreviewImageData.Num())
					{
						SendBuffer.Reset(BufferSizePerChunk);
						
						// Figure how much needs to be sent (could be less than the chunk size)
						int32 BytesToSend = FMath::Min(BufferSizePerChunk, SharedGameState->PreviewImageData.Num() - CurrentOffset);
						SendBuffer.AddUninitialized(BytesToSend);
						FMemory::Memcpy((void*)SendBuffer.GetData(), (void*)(SharedGameState->PreviewImageData.GetData() + CurrentOffset), BytesToSend);
						
						PC->ClientUpdatePreviewImageData(CurrentOffset, SendBuffer);
						ReplState.PreviewImageOffset += BytesToSend;

						UE_LOGF(LogAR, Verbose, "Sent ClientUpdatePreviewImageData(%d, %d) to PC (%ls)", CurrentOffset, BytesToSend, *PC->GetName());
					}
					// See if we need to send any AR world data
					CurrentOffset = ReplState.ARWorldOffset;
					if (CurrentOffset < SharedGameState->ARWorldData.Num())
					{
						SendBuffer.Reset(BufferSizePerChunk);
						
						// Figure how much needs to be sent (could be less than the chunk size)
						int32 BytesToSend = FMath::Min(BufferSizePerChunk, SharedGameState->ARWorldData.Num() - CurrentOffset);
						SendBuffer.AddUninitialized(BytesToSend);
						FMemory::Memcpy((void*)SendBuffer.GetData(), (void*)(SharedGameState->ARWorldData.GetData() + CurrentOffset), BytesToSend);
						
						PC->ClientUpdateARWorldData(CurrentOffset, SendBuffer);
						ReplState.ARWorldOffset += BytesToSend;

						UE_LOGF(LogAR, Verbose, "Sent ClientUpdateARWorldData(%d, %d) to PC (%ls)", CurrentOffset, BytesToSend, *PC->GetName());
					}
				}
				else
				{
					// Add if we haven't seen this player before
					PlayerToReplicationStateMap.Add(PC, FARSharedWorldReplicationState());
					PC->ClientInitSharedWorld(SharedGameState->PreviewImageData.Num(), SharedGameState->ARWorldData.Num());
					UE_LOGF(LogAR, Verbose, "Sent InitSharedWorld(%d, %d) to PC (%ls)", SharedGameState->PreviewImageData.Num(), SharedGameState->ARWorldData.Num(), *PC->GetName());
				}
			}
		}
	}
}

void AARSharedWorldGameMode::Logout(AController* Exiting)
{
	Super::Logout(Exiting);
	
	AARSharedWorldPlayerController* PC = Cast<AARSharedWorldPlayerController>(Exiting);
	if (PC != nullptr && PlayerToReplicationStateMap.Contains(PC))
	{
		UE_LOGF(LogAR, Verbose, "Removing PC (%ls)", *PC->GetName());

		PlayerToReplicationStateMap.Remove(PC);
	}
}

