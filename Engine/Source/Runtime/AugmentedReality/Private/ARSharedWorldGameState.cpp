// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSharedWorldGameState.h"
#include "AugmentedRealityModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARSharedWorldGameState)

AARSharedWorldGameState::AARSharedWorldGameState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PreviewImageBytesTotal(0)
	, ARWorldBytesTotal(0)
	, PreviewImageBytesDelivered(0)
	, ARWorldBytesDelivered(0)
	, bFiredCompletionEvent(false)
{
	
}

void AARSharedWorldGameState::InitSharedWorld(int32 PreviewImageSize, int32 ARWorldDataSize)
{
	// Should be called on the client only
	if (GIsServer)
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::InitSharedWorld() was called on the server. This is client only");
		return;
	}

	ARWorldBytesTotal = 0;
	PreviewImageBytesTotal = 0;
	ARWorldBytesDelivered = 0;
	PreviewImageBytesDelivered = 0;


	if (PreviewImageSize > 0 && ARWorldDataSize > 0)
	{
		PreviewImageData.Reset(PreviewImageSize);
		PreviewImageData.AddUninitialized(PreviewImageSize);
		PreviewImageBytesTotal = PreviewImageSize;
		
		ARWorldData.Reset(ARWorldDataSize);
		ARWorldData.AddUninitialized(ARWorldDataSize);
		ARWorldBytesTotal = ARWorldDataSize;
	}
	else
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::InitSharedWorld() was called with invalid sizes (%d, %d)", PreviewImageSize, ARWorldDataSize);
	}
}

void AARSharedWorldGameState::UpdatePreviewImageData(int32 Offset, const uint8* Buffer, int32 Size)
{
	// Should be called on the client only
	if (GIsServer)
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::UpdatePreviewImageData() was called on the server. This is client only");
		return;
	}
	
	// Validate offset, and also check for signed integer overflow case.
	if (Offset >= 0 && (Offset + Size) <= PreviewImageBytesTotal && (Offset + Size) >= 0)
	{
		uint8* PreviewImageBuffer = PreviewImageData.GetData();
		FMemory::Memcpy((void*)&PreviewImageBuffer[Offset], (void*)Buffer, Size);
		PreviewImageBytesDelivered += Size;

		// Trigger the completion event
		TriggerCompletionIfDone();
	}
	else
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::UpdatePreviewImageData() was called with bad offset (%d) or size (%d)", Offset, Size);
	}
}

void AARSharedWorldGameState::UpdateARWorldData(int32 Offset, const uint8* Buffer, int32 Size)
{
	// Should be called on the client only
	if (GIsServer)
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::UpdateARWorldData() was called on the server. This is client only");
		return;
	}
	
	// Validate offset, and also check for signed integer overflow case.
	if (Offset >= 0 && (Offset + Size) <= ARWorldBytesTotal && (Offset + Size) >= 0)
	{
		uint8* ARWorldBuffer = ARWorldData.GetData();
		FMemory::Memcpy((void*)&ARWorldBuffer[Offset], (void*)Buffer, Size);
		ARWorldBytesDelivered += Size;
		
		// Trigger the completion event
		TriggerCompletionIfDone();
	}
	else
	{
		UE_LOGF(LogAR, Warning, "AARSharedWorldGameState::UpdateARWorldData() was called with bad offset (%d) or size (%d)", Offset, Size);
	}
}

void AARSharedWorldGameState::TriggerCompletionIfDone()
{
	if (!bFiredCompletionEvent && ARWorldBytesDelivered == ARWorldBytesTotal && PreviewImageBytesDelivered == PreviewImageBytesTotal)
	{
		UE_LOGF(LogAR, Log, "Notifying client AR world data is ready");

		bFiredCompletionEvent = true;
		K2_OnARWorldMapIsReady();
	}
}

