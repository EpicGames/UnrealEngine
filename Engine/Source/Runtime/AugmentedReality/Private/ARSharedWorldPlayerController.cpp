// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSharedWorldPlayerController.h"
#include "ARSharedWorldGameState.h"
#include "Engine/World.h"
#include "AugmentedRealityModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARSharedWorldPlayerController)

AARSharedWorldPlayerController::AARSharedWorldPlayerController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bIsReadyToReceive(false)
{
	
}

AARSharedWorldGameState* AARSharedWorldPlayerController::GetGameState()
{
	UWorld* World = GetWorld();
	AARSharedWorldGameState* GameState = World != nullptr ? CastChecked<AARSharedWorldGameState>(World->GetGameState()) : nullptr;
	return GameState;
}

bool AARSharedWorldPlayerController::IsGameStateReady()
{
	UWorld* World = GetWorld();
	AARSharedWorldGameState* GameState = World != nullptr ? Cast<AARSharedWorldGameState>(World->GetGameState()) : nullptr;
	return GameState != nullptr;
}

void AARSharedWorldPlayerController::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	// If we are the client, check that our game state has replicated over
	if (!HasAuthority() && !bIsReadyToReceive && IsGameStateReady())
	{
		bIsReadyToReceive = true;
		ServerMarkReadyForReceiving();

		UE_LOGF(LogAR, Verbose, "Notifying server ready to receive via ServerMarkReadyForReceiving()");
	}
}

bool AARSharedWorldPlayerController::ServerMarkReadyForReceiving_Validate()
{
	return true;
}

void AARSharedWorldPlayerController::ServerMarkReadyForReceiving_Implementation()
{
	bIsReadyToReceive = true;
	UE_LOGF(LogAR, Verbose, "Client is ready to receive");
}

bool AARSharedWorldPlayerController::ClientInitSharedWorld_Validate(int32 PreviewImageSize, int32 ARWorldDataSize)
{
	return PreviewImageSize >= 0 && PreviewImageSize <= (8 * 1024 * 1024) && ARWorldDataSize >= 0 && ARWorldDataSize <= (128 * 1024 * 1024);
}

void AARSharedWorldPlayerController::ClientInitSharedWorld_Implementation(int32 PreviewImageSize, int32 ARWorldDataSize)
{
	UE_LOGF(LogAR, Verbose, "Client received ClientInitSharedWorld(%d, %d)", PreviewImageSize, ARWorldDataSize);

	GetGameState()->InitSharedWorld(PreviewImageSize, ARWorldDataSize);
}

bool AARSharedWorldPlayerController::ClientUpdatePreviewImageData_Validate(int32 Offset, const TArray<uint8>& Buffer)
{
	// Validate offset, and also check for signed integer overflow case.
	return Offset >= 0 && (Offset + Buffer.Num()) >= 0;
}

void AARSharedWorldPlayerController::ClientUpdatePreviewImageData_Implementation(int32 Offset, const TArray<uint8>& Buffer)
{
	UE_LOGF(LogAR, Verbose, "Client received ClientUpdatePreviewImageData(%d, %d)", Offset, Buffer.Num());

	GetGameState()->UpdatePreviewImageData(Offset, Buffer.GetData(), Buffer.Num());
}

bool AARSharedWorldPlayerController::ClientUpdateARWorldData_Validate(int32 Offset, const TArray<uint8>& Buffer)
{
	// Validate offset, and also check for signed integer overflow case.
	return Offset >= 0 && (Offset + Buffer.Num()) >= 0;
}

void AARSharedWorldPlayerController::ClientUpdateARWorldData_Implementation(int32 Offset, const TArray<uint8>& Buffer)
{
	UE_LOGF(LogAR, Verbose, "Client received ClientUpdateARWorldData(%d, %d)", Offset, Buffer.Num());

	GetGameState()->UpdateARWorldData(Offset, Buffer.GetData(), Buffer.Num());
}

