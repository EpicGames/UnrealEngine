// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSkyLight.h"
#include "ARTrackable.h"
#include "ARTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARSkyLight)

#define LOCTEXT_NAMESPACE "ARSkyLight"

AARSkyLight::AARSkyLight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, CaptureProbe(nullptr)
	, LastUpdateTimestamp(0.f)
{
	PrimaryActorTick.bCanEverTick = true;
	// Don't start ticking until we have a valid capture probe to update with
	PrimaryActorTick.bStartWithTickEnabled = false;
	// The environment probes don't update often so don't check very often
	PrimaryActorTick.TickInterval = 0.25f;

	// We only work with environment probes that generate a cube map
	GetLightComponent()->SourceType = ESkyLightSourceType::SLS_SpecifiedCubemap;
	GetLightComponent()->bLowerHemisphereIsBlack = false;
}

// @todo - JoeG add double buffering of the cube map textures so we can blend the updates rather than popping
void AARSkyLight::Tick(float DeltaTime)
{
	if (CaptureProbe == nullptr)
	{
		// Disable our tick because there's nothing to check
		SetActorTickEnabled(false);
		return;
	}

	// Check to see if the probe was updated by the ARSystem
	if (LastUpdateTimestamp < CaptureProbe->GetLastUpdateTimestamp())
	{
		// Trigger a refresh of the cube map data
		GetLightComponent()->MarkRenderStateDirty();
		GetLightComponent()->SetCaptureIsDirty();
		LastUpdateTimestamp = CaptureProbe->GetLastUpdateTimestamp();
	}
}

void AARSkyLight::SetEnvironmentCaptureProbe(UAREnvironmentCaptureProbe* InCaptureProbe)
{
	CaptureProbe = InCaptureProbe;
	// Turn ticking on if we have a capture probe that needs regular updating
	SetActorTickEnabled(CaptureProbe != nullptr);

	if (CaptureProbe != nullptr)
	{
		UTextureCube* CubeTexture = CaptureProbe->GetEnvironmentCaptureTexture();
		GetLightComponent()->SetCubemap(CubeTexture);
		LastUpdateTimestamp = CaptureProbe->GetLastUpdateTimestamp();
	}
	else
	{
		LastUpdateTimestamp = 0.f;
	}
}

#undef LOCTEXT_NAMESPACE

