// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSupportInterface.h"
#include "ARTraceResult.h"
#include "Features/IModularFeatures.h"
#include "ARBlueprintLibrary.h"
#include "ARBlueprintProxy.h"
#include "Templates/SharedPointer.h"
#include "Engine/Texture2D.h"

FARSupportInterface::FARSupportInterface (IARSystemSupport* InARImplementation, IXRTrackingSystem* InXRTrackingSystem)
	: ARImplemention(InARImplementation)
	, XRTrackingSystem(InXRTrackingSystem)
	, AlignmentTransform(FTransform::Identity)
	, ARSettings(NewObject<UARSessionConfig>())
{
}

FARSupportInterface ::~FARSupportInterface ()
{
	IModularFeatures::Get().UnregisterModularFeature(FARSupportInterface ::GetModularFeatureName(), this);
}

void FARSupportInterface ::InitializeARSystem()
{
	// Register our ability to support Unreal AR API.
	IModularFeatures::Get().RegisterModularFeature(FARSupportInterface ::GetModularFeatureName(), this);

	if (ARImplemention)
	{
		UARBlueprintLibrary::RegisterAsARSystem(AsShared());
		UARBaseAsyncTaskBlueprintProxy::RegisterAsARSystem(AsShared());

		ARImplemention->OnARSystemInitialized();
	}
}

IXRTrackingSystem* FARSupportInterface ::GetXRTrackingSystem()
{
	return XRTrackingSystem;
}

const FTransform& FARSupportInterface ::GetAlignmentTransform() const
{
	return AlignmentTransform;
}

const UARSessionConfig& FARSupportInterface ::GetSessionConfig() const
{
	check(ARSettings != nullptr);
	return *ARSettings;
}

UARSessionConfig& FARSupportInterface ::AccessSessionConfig()
{
	check(ARSettings != nullptr);
	return *ARSettings;
}

bool FARSupportInterface ::StartARGameFrame(FWorldContext& WorldContext)
{
	if (ARImplemention)
	{
		return ARImplemention->OnStartARGameFrame(WorldContext);
	}
	return false;
}


EARTrackingQuality FARSupportInterface ::GetTrackingQuality() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetTrackingQuality();
	}
	return EARTrackingQuality::NotTracking;
}

EARTrackingQualityReason FARSupportInterface::GetTrackingQualityReason() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetTrackingQualityReason();
	}
	return EARTrackingQualityReason::None;
}

void FARSupportInterface ::StartARSession(UARSessionConfig* InSessionConfig)
{
	if (ARImplemention)
	{
		ARSettings = InSessionConfig;
		ARImplemention->OnStartARSession(InSessionConfig);
	}
}

void FARSupportInterface ::PauseARSession()
{
	if (ARImplemention)
	{
		if (GetARSessionStatus().Status == EARSessionStatus::Running)
		{
			ARImplemention->OnPauseARSession();
		}
	}
}

void FARSupportInterface ::StopARSession()
{
	if (ARImplemention)
	{
		//Removing check allows for extra safeguards to close down during a run.
		//if (GetARSessionStatus().Status == EARSessionStatus::Running)
		{
			ARImplemention->OnStopARSession();
		}
	}
}

FARSessionStatus FARSupportInterface ::GetARSessionStatus() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetARSessionStatus();
	}
	return EARSessionStatus::NotSupported;
}

bool FARSupportInterface ::IsSessionTypeSupported(EARSessionType SessionType) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnIsTrackingTypeSupported(SessionType);
	}
	return false;
}

bool FARSupportInterface::ToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType)
{
	if (ARImplemention)
	{
		return ARImplemention->OnToggleARCapture(bOnOff, CaptureType);
	}
	return false;
}


void FARSupportInterface::SetEnabledXRCamera(bool bOnOff)
{
	if (ARImplemention)
	{
		ARImplemention->OnSetEnabledXRCamera(bOnOff);
	}
}

FIntPoint FARSupportInterface::ResizeXRCamera(const FIntPoint& InSize)
{
	if (ARImplemention)
	{
		return ARImplemention->OnResizeXRCamera(InSize);
	}
	return FIntPoint(0, 0);
}


void FARSupportInterface::SetAlignmentTransform(const FTransform& InAlignmentTransform)
{
	if (ARImplemention)
	{
		ARImplemention->OnSetAlignmentTransform(InAlignmentTransform);
	}
	AlignmentTransform = InAlignmentTransform;
	OnAlignmentTransformUpdated.Broadcast(InAlignmentTransform);
}

TArray<FARTraceResult> FARSupportInterface ::LineTraceTrackedObjects(const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels)
{
	if (ARImplemention)
	{
		return ARImplemention->OnLineTraceTrackedObjects(ScreenCoord, TraceChannels);
	}
	return TArray<FARTraceResult>();
}

TArray<FARTraceResult> FARSupportInterface::LineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels)
{
	if (ARImplemention)
	{
		return ARImplemention->OnLineTraceTrackedObjects(Start, End, TraceChannels);
	}
	return TArray<FARTraceResult>();

}

TArray<UARTrackedGeometry*> FARSupportInterface ::GetAllTrackedGeometries() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetAllTrackedGeometries();
	}
	return TArray<UARTrackedGeometry*>();
}

TArray<UARPin*> FARSupportInterface ::GetAllPins() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetAllPins();
	}
	return TArray<UARPin*>();
}

bool FARSupportInterface ::AddManualEnvironmentCaptureProbe(FVector Location, FVector Extent)
{
	if (ARImplemention)
	{
		return ARImplemention->OnAddManualEnvironmentCaptureProbe(Location, Extent);
	}
	return false;
}

TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> FARSupportInterface ::GetCandidateObject(FVector Location, FVector Extent) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetCandidateObject(Location, Extent);
	}
	return TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe>();
}

TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> FARSupportInterface ::SaveWorld() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnSaveWorld();
	}
	return TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe>();
}

EARWorldMappingState FARSupportInterface ::GetWorldMappingStatus() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetWorldMappingStatus();
	}
	return EARWorldMappingState::NotAvailable;
}


UARLightEstimate* FARSupportInterface ::GetCurrentLightEstimate() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetCurrentLightEstimate();
	}
	return nullptr;
}

UARPin* FARSupportInterface ::PinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry, const FName DebugName)
{
	if (ARImplemention)
	{
		return ARImplemention->OnPinComponent(ComponentToPin, PinToWorldTransform, TrackedGeometry, DebugName);
	}
	return nullptr;
}

UARPin* FARSupportInterface ::PinComponent(USceneComponent* ComponentToPin, const FARTraceResult& HitResult, const FName DebugName)
{
	if (ARImplemention)
	{
		return ARImplemention->OnPinComponent(ComponentToPin, HitResult.GetLocalToWorldTransform(), HitResult.GetTrackedGeometry(), DebugName);
	}
	return nullptr;
}

void FARSupportInterface ::RemovePin(UARPin* PinToRemove)
{
	if (ARImplemention)
	{
		ARImplemention->OnRemovePin(PinToRemove);
	}
}

bool FARSupportInterface ::TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin)
{
	OutPin = nullptr;
	if (ARImplemention)
	{
		return ARImplemention->OnTryGetOrCreatePinForNativeResource(InNativeResource, InPinName, OutPin);
	}

	return false;
}

TArray<FARVideoFormat> FARSupportInterface ::GetSupportedVideoFormats(EARSessionType SessionType) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetSupportedVideoFormats(SessionType);
	}
	return TArray<FARVideoFormat>();
}

/** @return the current point cloud data for the ar scene */
TArray<FVector> FARSupportInterface ::GetPointCloud() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetPointCloud();
	}
	return TArray<FVector>();
}

UARCandidateImage* FARSupportInterface::AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth)
{
	if (ARImplemention && ARImplemention->OnAddRuntimeCandidateImage(SessionConfig, CandidateTexture, FriendlyName, PhysicalWidth))
	{
		float PhysicalHeight = PhysicalWidth / FMath::Max<int32>(1, CandidateTexture->GetSizeX()) * CandidateTexture->GetSizeY();
		UARCandidateImage* NewCandidateImage = UARCandidateImage::CreateNewARCandidateImage(CandidateTexture, FriendlyName, PhysicalWidth, PhysicalHeight, EARCandidateImageOrientation::Landscape);
		SessionConfig->AddCandidateImage(NewCandidateImage);
		return NewCandidateImage;
	}
	else
	{
		return nullptr;
	}
}

void* FARSupportInterface ::GetARSessionRawPointer()
{
	if (ARImplemention)
	{
		return ARImplemention->GetARSessionRawPointer();
	}
	return nullptr;
}

void* FARSupportInterface ::GetGameThreadARFrameRawPointer()
{
	if (ARImplemention)
	{
		return ARImplemention->GetGameThreadARFrameRawPointer();
	}
	return nullptr;
}


void FARSupportInterface ::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (ARSettings != nullptr)
	{
		Collector.AddReferencedObject(ARSettings);
	}
}

bool FARSupportInterface::PinComponent(USceneComponent* ComponentToPin, UARPin* Pin)
{
	if (ARImplemention)
	{
		return ARImplemention->OnPinComponentToARPin(ComponentToPin, Pin);
	}
	return false;
}

bool FARSupportInterface::IsLocalPinSaveSupported() const
{
	if (ARImplemention)
	{
		return ARImplemention->IsLocalPinSaveSupported();
	}
	return false;
}

bool FARSupportInterface::ArePinsReadyToLoad()
{
	if (ARImplemention)
	{
		return ARImplemention->ArePinsReadyToLoad();
	}
	return false;
}

void FARSupportInterface::LoadARPins(TMap<FName, UARPin*>& LoadedPins)
{
	if (ARImplemention)
	{
		return ARImplemention->LoadARPins(LoadedPins);
	}
}

bool FARSupportInterface::SaveARPin(FName InName, UARPin* InPin)
{
	if (ARImplemention)
	{
		return ARImplemention->SaveARPin(InName, InPin);
	}
	return false;
}

void FARSupportInterface::RemoveSavedARPin(FName InName)
{
	if (ARImplemention)
	{
		return ARImplemention->RemoveSavedARPin(InName);
	}
}
void FARSupportInterface::RemoveAllSavedARPins()
{
	if (ARImplemention)
	{
		return ARImplemention->RemoveAllSavedARPins();
	}
}


bool FARSupportInterface::IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnIsSessionTrackingFeatureSupported(SessionType, SessionTrackingFeature);
	}
	return false;
}

TArray<FARPose2D> FARSupportInterface::GetTracked2DPose() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetTracked2DPose();
	}
	return {};
}

bool FARSupportInterface::IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnIsSceneReconstructionSupported(SessionType, SceneReconstructionMethod);
	}
	return false;
}

bool FARSupportInterface::AddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName)
{
	if (ARImplemention)
	{
		return ARImplemention->OnAddTrackedPointWithName(WorldTransform, PointName, bDeletePointsWithSameName);
	}
	return false;
}

int32 FARSupportInterface::GetNumberOfTrackedFacesSupported() const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetNumberOfTrackedFacesSupported();
	}
	return 0;
}

UARTexture* FARSupportInterface::GetARTexture(EARTextureType TextureType) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetARTexture(TextureType);
	}
	return nullptr;
}

bool FARSupportInterface::GetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const
{
	if (ARImplemention)
	{
		return ARImplemention->OnGetCameraIntrinsics(OutCameraIntrinsics);
	}
	return false;
}

bool FARSupportInterface::IsARAvailable() const
{
	if (ARImplemention)
	{
		return ARImplemention->IsARAvailable();
	}
	return false;
}

#define DEFINE_AR_SI_DELEGATE_FUNCS(DelegateName) \
FDelegateHandle FARSupportInterface::Add##DelegateName##Delegate_Handle(const F##DelegateName##Delegate& Delegate) \
{ \
	if (ARImplemention) \
	{ \
		return ARImplemention->Add##DelegateName##Delegate_Handle(Delegate); \
	} \
	return Delegate.GetHandle(); \
} \
void FARSupportInterface::Clear##DelegateName##Delegate_Handle(FDelegateHandle& Handle) \
{ \
	if (ARImplemention) \
	{ \
		ARImplemention->Clear##DelegateName##Delegate_Handle(Handle); \
		return; \
	} \
	Handle.Reset(); \
} \
void FARSupportInterface::Clear##DelegateName##Delegates(FDelegateUserObject Object) \
{ \
	if (ARImplemention) \
	{ \
		ARImplemention->Clear##DelegateName##Delegates(Object); \
	} \
}

DEFINE_AR_SI_DELEGATE_FUNCS(OnTrackableAdded)
DEFINE_AR_SI_DELEGATE_FUNCS(OnTrackableUpdated)
DEFINE_AR_SI_DELEGATE_FUNCS(OnTrackableRemoved)
