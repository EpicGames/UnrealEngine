// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARSystemSupportBase.h"
#include "AugmentedRealityModule.h"
#include "ARPin.h"

bool FARSystemSupportBase::OnPinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin)
{
	if (Pin == nullptr)
	{
		UE_LOGF(LogAR, Warning, "OnPinComponentToARPin: Pin was null.  Doing nothing.");
		return false;
	}
	if (ComponentToPin == nullptr)
	{
		UE_LOGF(LogAR, Warning, "OnPinComponentToARPin: Tried to pin null component to pin %ls.  Doing nothing.", *Pin->GetDebugName().ToString());
		return false;
	}

	{
		if (UARPin* FindResult = FindARPinByComponent(ComponentToPin))
		{
			if (FindResult == Pin)
			{
				UE_LOGF(LogAR, Warning, "OnPinComponentToARPin: Component %ls is already pinned to pin %ls.  Doing nothing.", *ComponentToPin->GetReadableName(), *Pin->GetDebugName().ToString());
				return true;
			}
			else
			{
				UE_LOGF(LogAR, Warning, "OnPinComponentToARPin: Component %ls is pinned to pin %ls. Unpinning it from that pin first.  The pin will not be destroyed.", *ComponentToPin->GetReadableName(), *Pin->GetDebugName().ToString());
				FindResult->SetPinnedComponent(nullptr);
			}
		}

		Pin->SetPinnedComponent(ComponentToPin);

		return true;
	}
}
