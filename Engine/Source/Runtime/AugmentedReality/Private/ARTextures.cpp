// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARTextures.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARTextures)

UARTexture::UARTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UARTextureCameraImage::UARTextureCameraImage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TextureType = EARTextureType::CameraImage;
}

UARTextureCameraDepth::UARTextureCameraDepth(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	TextureType = EARTextureType::CameraDepth;
}

UAREnvironmentCaptureProbeTexture::UAREnvironmentCaptureProbeTexture(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TextureType(EARTextureType::EnvironmentCapture)
{
}

