// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARTraceResult.h"
#include "ARSystem.h"
#include "ARSupportInterface.h"
#include "IXRTrackingSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARTraceResult)


//
//
//
FARTraceResult::FARTraceResult()
: FARTraceResult(nullptr, 0.0f, EARLineTraceChannels::None, FTransform(), nullptr)
{
	
}


FARTraceResult::FARTraceResult( const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& InARSystem, float InDistanceFromCamera, EARLineTraceChannels InTraceChannel, const FTransform& InLocalTransform, UARTrackedGeometry* InTrackedGeometry )
: DistanceFromCamera(InDistanceFromCamera)
, TraceChannel(InTraceChannel)
, LocalTransform(InLocalTransform)
, TrackedGeometry(InTrackedGeometry)
, ARSystem(InARSystem)
{
	
}

float FARTraceResult::GetDistanceFromCamera() const
{
	return DistanceFromCamera;
}

void FARTraceResult::SetLocalToWorldTransform(const FTransform& LocalToWorldTransform)
{
	const auto TrackingToWorldTransform = ARSystem->GetXRTrackingSystem()->GetTrackingToWorldTransform();
	const auto AlignmentTransform = ARSystem->GetAlignmentTransform();
	LocalTransform = LocalToWorldTransform * TrackingToWorldTransform.Inverse() * AlignmentTransform.Inverse();
}

FTransform FARTraceResult::GetLocalToTrackingTransform() const
{
	return LocalTransform * ARSystem->GetAlignmentTransform();
}

FTransform FARTraceResult::GetLocalToWorldTransform() const
{
	return GetLocalToTrackingTransform() * ARSystem->GetXRTrackingSystem()->GetTrackingToWorldTransform();
}

FTransform FARTraceResult::GetLocalTransform() const
{
	return LocalTransform;
}

UARTrackedGeometry* FARTraceResult::GetTrackedGeometry() const
{
	return TrackedGeometry;
}

EARLineTraceChannels FARTraceResult::GetTraceChannel() const
{
	return TraceChannel;
}

