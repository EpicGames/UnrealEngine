// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARTrackable.h"
#include "ARSupportInterface.h"
#include "ARSystem.h"
#include "ARDebugDrawHelpers.h"
#include "DrawDebugHelpers.h"
#include "MRMeshComponent.h"
#include "IXRTrackingSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARTrackable)

//
//
//
UARTrackedGeometry::UARTrackedGeometry()
:TrackingState(EARTrackingState::Tracking)
,NativeResource(nullptr)
{
	
}

void UARTrackedGeometry::InitializeNativeResource(IARRef* InNativeResource)
{
	NativeResource.Reset(InNativeResource);
}

void UARTrackedGeometry::DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds ) const
{
	const FTransform WorldTrans = GetLocalToWorldTransform();
	const FVector Location = WorldTrans.GetLocation();
	const FRotator Rotation = FRotator(WorldTrans.GetRotation());
	const FVector Scale3D = WorldTrans.GetScale3D();
	DrawDebugCoordinateSystem(World, Location, Rotation, (float)Scale3D.X, true, PersistForSeconds, 0, OutlineThickness);
}

void UARTrackedGeometry::GetNetworkPayload(FARMeshUpdatePayload& Payload)
{
	Payload.WorldTransform = GetLocalToWorldTransform();
	Payload.ObjectClassification = ObjectClassification;
	UpdateSessionPayload(Payload.SessionPayload);
}

TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> UARTrackedGeometry::GetARSystem() const
{
	auto MyARSystem = ARSystem.Pin();
	return MyARSystem;
}

void UARTrackedGeometry::UpdateSessionPayload(FARSessionPayload& Payload) const
{
	if (auto MyARSystem = ARSystem.Pin())
	{
		Payload.FromSessionConfig(MyARSystem->GetSessionConfig());
	}
}

FTransform UARTrackedGeometry::GetLocalToTrackingTransform() const
{
	return LocalToAlignedTrackingTransform;
}

FTransform UARTrackedGeometry::GetLocalToTrackingTransform_NoAlignment() const
{
	return LocalToTrackingTransform;
}

EARTrackingState UARTrackedGeometry::GetTrackingState() const
{
	return TrackingState;
}

bool UARTrackedGeometry::IsTracked() const
{
	return TrackingState == EARTrackingState::Tracking;
}

void UARTrackedGeometry::SetTrackingState(EARTrackingState NewState)
{
	UpdateTrackingState(NewState);
}

FTransform UARTrackedGeometry::GetLocalToWorldTransform() const
{
	if (auto XRSystem = GEngine->XRSystem.Get())
	{
		return GetLocalToTrackingTransform() * XRSystem->GetTrackingToWorldTransform();
	}
	else
	{
		return GetLocalToTrackingTransform();
	}
}

int32 UARTrackedGeometry::GetLastUpdateFrameNumber() const
{
	return LastUpdateFrameNumber;
}

FName UARTrackedGeometry::GetDebugName() const
{
	return DebugName;
}

const FString& UARTrackedGeometry::GetName() const
{
	return AnchorName;
}

void UARTrackedGeometry::SetName(const FString& InName)
{
	AnchorName = InName;
}

float UARTrackedGeometry::GetLastUpdateTimestamp() const
{
	return (float)LastUpdateTimestamp;
}
void UARTrackedGeometry::UpdateTrackedGeometryNoMove(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp)
{
	ARSystem = InTrackingSystem;
	LastUpdateFrameNumber = FrameNumber;
	LastUpdateTimestamp = Timestamp;
	SetTrackingState(EARTrackingState::Tracking);
}

void UARTrackedGeometry::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform )
{
	ARSystem = InTrackingSystem;
	LocalToTrackingTransform = InLocalToTrackingTransform;
	LastUpdateFrameNumber = FrameNumber;
	LastUpdateTimestamp = Timestamp;
	UpdateAlignmentTransform(InAlignmentTransform);
	// We were updated, so we're clearly being tracked ;)
	SetTrackingState(EARTrackingState::Tracking);
}

void UARTrackedGeometry::UpdateTrackingState( EARTrackingState NewTrackingState )
{
	TrackingState = NewTrackingState;

	if (TrackingState == EARTrackingState::StoppedTracking)
	{
		if (NativeResource.IsValid())
		{
			// Remove reference to the native resource since the tracked geometry is stopped tracking.
			NativeResource->RemoveRef();
		}
	}
}

void UARTrackedGeometry::UpdateAlignmentTransform( const FTransform& NewAlignmentTransform )
{
	LocalToAlignedTrackingTransform = LocalToTrackingTransform * NewAlignmentTransform;
	
	if (UnderlyingMesh)
	{
		// Update MR mesh with the new local to world transform as it needs to be at the same location of this geometry
		UnderlyingMesh->SetWorldTransform(GetLocalToWorldTransform());
	}
}

void UARTrackedGeometry::SetDebugName( FName InDebugName )
{
	DebugName = InDebugName;
}

IARRef* UARTrackedGeometry::GetNativeResource()
{
	return NativeResource.Get();
}

UMRMeshComponent* UARTrackedGeometry::GetUnderlyingMesh()
{
	return UnderlyingMesh;
}

void UARTrackedGeometry::SetUnderlyingMesh(UMRMeshComponent* InMRMeshComponent)
{
	UnderlyingMesh = InMRMeshComponent;
}

//
//
//
void UARPlaneGeometry::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FVector InCenter, const FVector InExtent )
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	Center = InCenter;
	Extent = InExtent;
	
	BoundaryPolygon.Empty(4);
	BoundaryPolygon.Add(FVector(-Extent.X, -Extent.Y, 0.0f));
	BoundaryPolygon.Add(FVector(Extent.X, -Extent.Y, 0.0f));
	BoundaryPolygon.Add(FVector(Extent.X, Extent.Y, 0.0f));
	BoundaryPolygon.Add(FVector(-Extent.X, Extent.Y, 0.0f));

	SubsumedBy = nullptr;
}

void UARPlaneGeometry::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FVector InCenter, const FVector InExtent, const TArray<FVector>& InBoundingPoly, UARPlaneGeometry* InSubsumedBy)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	Center = InCenter;
	Extent = InExtent;

	BoundaryPolygon = InBoundingPoly;

	SubsumedBy = InSubsumedBy;
}

void UARPlaneGeometry::DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds ) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FColor OutlineRGB = OutlineColor.ToFColor(false);
	if (BoundaryPolygon.Num() > 2)
	{
		FVector LastVert = LocalToWorldTransform.TransformPosition(BoundaryPolygon[0]);
		for (int32 i=1; i<BoundaryPolygon.Num(); ++i)
		{
			const FVector NewVert = LocalToWorldTransform.TransformPosition(BoundaryPolygon[i]);
			DrawDebugLine(World, LastVert, NewVert, OutlineRGB, PersistForSeconds > 0 ? true : false, PersistForSeconds, 0, OutlineThickness);
			LastVert = NewVert;
		}
		DrawDebugLine(World, LastVert, LocalToWorldTransform.TransformPosition(BoundaryPolygon[0]), OutlineRGB, PersistForSeconds > 0 ? true : false, PersistForSeconds, 0, OutlineThickness);
	}

	const FVector WorldSpaceCenter = LocalToWorldTransform.TransformPosition(Center);
	DrawDebugBox( World, WorldSpaceCenter, Extent, LocalToWorldTransform.GetRotation(), OutlineRGB, false, PersistForSeconds, 0, 0.1f*OutlineThickness );
	
	const FString CurAnchorDebugName = GetDebugName().ToString();
	ARDebugHelpers::DrawDebugString( World, WorldSpaceCenter, CurAnchorDebugName, 0.25f*OutlineThickness, OutlineRGB, PersistForSeconds, true);
}

void UARPlaneGeometry::GetNetworkPayload(FARPlaneUpdatePayload& Payload)
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();

	Payload.WorldTransform = LocalToWorldTransform;
	Payload.Center = Center;
	Payload.Extents = Extent;
	Payload.BoundaryVertices = BoundaryPolygon;
	Payload.ObjectClassification = ObjectClassification;
	
	UpdateSessionPayload(Payload.SessionPayload);
}

void UARTrackedImage::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds /*= 0.0f*/) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FString CurAnchorDebugName = FString::Printf(TEXT("%s - %s"), *GetDebugName().ToString(), *DetectedImage->GetFriendlyName());
	const FColor OutlineRGB = OutlineColor.ToFColor(false);

	FVector Extent(DetectedImage->GetPhysicalHeight() / 2.f, DetectedImage->GetPhysicalWidth() / 2.f, 0.f);
	
	const FVector WorldSpaceCenter = LocalToWorldTransform.GetLocation();
	DrawDebugBox(World, WorldSpaceCenter, Extent, LocalToWorldTransform.GetRotation(), OutlineRGB, false, PersistForSeconds, 0, 0.1f * OutlineThickness);

	ARDebugHelpers::DrawDebugString(World, WorldSpaceCenter, CurAnchorDebugName, 0.25f * OutlineThickness, OutlineRGB, PersistForSeconds, true);
}

void UARTrackedImage::GetNetworkPayload(FARImageUpdatePayload& Payload)
{
	Payload.WorldTransform = GetLocalToWorldTransform();
	Payload.DetectedImage = DetectedImage;
	Payload.EstimatedSize = EstimatedSize;
	
	UpdateSessionPayload(Payload.SessionPayload);
}

void UARTrackedImage::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector2D InEstimatedSize, UARCandidateImage* InDetectedImage)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	EstimatedSize = InEstimatedSize;
	DetectedImage = InDetectedImage;
	ObjectClassification = EARObjectClassification::Image;
	
	if ((EstimatedSize.X == 0.f || EstimatedSize.Y == 0.f) && DetectedImage)
	{
		EstimatedSize.X = DetectedImage->GetPhysicalWidth();
		EstimatedSize.Y = DetectedImage->GetPhysicalHeight();
	}
}

FVector2D UARTrackedImage::GetEstimateSize()
{
	return EstimatedSize;
}

void UARTrackedQRCode::GetNetworkPayload(FARQRCodeUpdatePayload& Payload)
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();

	Payload.WorldTransform = LocalToWorldTransform;
	Payload.Extents = FVector(EstimatedSize.X, EstimatedSize.Y, 0.0f);

	Payload.QRCode = QRCode;
	
	UpdateSessionPayload(Payload.SessionPayload);
}

void UARTrackedQRCode::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector2D InEstimatedSize, const FString& CodeData, int32 InVersion)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform, InEstimatedSize, nullptr);
	QRCode = CodeData;
	Version = InVersion;
}

void UARFaceGeometry::UpdateFaceGeometry(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FARBlendShapeMap& InBlendShapes, TArray<FVector>& InVertices, const TArray<int32>& Indices, TArray<FVector2D>& InUVs, const FTransform& InLeftEyeTransform, const FTransform& InRightEyeTransform, const FVector& InLookAtTarget)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	BlendShapes = MoveTemp(InBlendShapes);
	VertexBuffer = MoveTemp(InVertices);
	UVs = MoveTemp(InUVs);
	// This won't change ever so only copy first time
	if (IndexBuffer.Num() == 0)
	{
		IndexBuffer = Indices;
	}
	
	LeftEyeTransform = InLeftEyeTransform;
	RightEyeTransform = InRightEyeTransform;
	LookAtTarget = InLookAtTarget;
	ObjectClassification = EARObjectClassification::Face;
}

void UARTrackedPoint::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds /*= 0.0f*/) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FString CurAnchorDebugName = GetDebugName().ToString();
	const FColor OutlineRGB =OutlineColor.ToFColor(false);
	ARDebugHelpers::DrawDebugString( World, LocalToWorldTransform.GetLocation(), CurAnchorDebugName, 0.25f*OutlineThickness, OutlineRGB, PersistForSeconds, true);
	
	DrawDebugPoint(World, LocalToWorldTransform.GetLocation(), 0.5f, OutlineRGB, false, PersistForSeconds, 0);
}

void UARTrackedPoint::GetNetworkPayload(FARPointUpdatePayload& Payload)
{
}

void UARTrackedPoint::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
}

void UARFaceGeometry::DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds ) const
{
	Super::DebugDraw(World, OutlineColor, OutlineThickness, PersistForSeconds);
}

void UARFaceGeometry::GetNetworkPayload(FARFaceUpdatePayload& Payload)
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();

	Payload.LeftEyePosition = LocalToWorldTransform.TransformPosition(LeftEyeTransform.GetLocation());
	Payload.RightEyePosition = LocalToWorldTransform.TransformPosition(RightEyeTransform.GetLocation());

	//not sure what space this is in...
	Payload.LookAtTarget = LocalToWorldTransform.TransformPosition(LookAtTarget);
	
	UpdateSessionPayload(Payload.SessionPayload);
}

float UARFaceGeometry::GetBlendShapeValue(EARFaceBlendShape BlendShape) const
{
	float Value = 0.f;
	if (BlendShapes.Contains(BlendShape))
	{
		Value = BlendShapes[BlendShape];
	}
	return Value;
}

const TMap<EARFaceBlendShape, float> UARFaceGeometry::GetBlendShapes() const
{
	return BlendShapes;
}

const FTransform& UARFaceGeometry::GetLocalSpaceEyeTransform(EAREye Eye) const
{
	if (Eye == EAREye::LeftEye)
	{
		return LeftEyeTransform;
	}
	return RightEyeTransform;
}

FTransform UARFaceGeometry::GetWorldSpaceEyeTransform(EAREye Eye) const
{
	if (Eye == EAREye::LeftEye)
	{
		return LeftEyeTransform * GetLocalToWorldTransform();
	}
	return RightEyeTransform * GetLocalToWorldTransform();
}

UAREnvironmentCaptureProbe::UAREnvironmentCaptureProbe()
	: Super()
	, EnvironmentCaptureTexture(nullptr)
{
}

void UAREnvironmentCaptureProbe::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FString CurAnchorDebugName = GetDebugName().ToString();
	const FColor OutlineRGB =OutlineColor.ToFColor(false);

	ARDebugHelpers::DrawDebugString( World, LocalToWorldTransform.GetLocation(), CurAnchorDebugName, 0.25f * OutlineThickness, OutlineRGB, PersistForSeconds, true );
	
	DrawDebugBox( World, LocalToWorldTransform.GetLocation(), Extent, LocalToWorldTransform.GetRotation(), OutlineRGB, false, PersistForSeconds, 0, 0.1f * OutlineThickness );
}

void UAREnvironmentCaptureProbe::GetNetworkPayload(FAREnvironmentProbeUpdatePayload& Payload)
{
}


void UAREnvironmentCaptureProbe::UpdateEnvironmentCapture(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 InFrameNumber, double InTimestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector InExtent)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, InFrameNumber, InTimestamp, InLocalToTrackingTransform, InAlignmentTransform);

	Extent = InExtent;
}

FVector UAREnvironmentCaptureProbe::GetExtent() const
{
	return Extent;
}

UAREnvironmentCaptureProbeTexture* UAREnvironmentCaptureProbe::GetEnvironmentCaptureTexture()
{
	return EnvironmentCaptureTexture;
}

void UARTrackedObject::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds /*= 0.0f*/) const
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FString CurAnchorDebugName = GetDebugName().ToString();
	const FColor OutlineRGB =OutlineColor.ToFColor(false);
	ARDebugHelpers::DrawDebugString( World, LocalToWorldTransform.GetLocation(), CurAnchorDebugName, 0.25f*OutlineThickness, OutlineRGB, PersistForSeconds, true);
	
	DrawDebugPoint(World, LocalToWorldTransform.GetLocation(), 0.5f, OutlineRGB, false, PersistForSeconds, 0);
}

void UARTrackedObject::GetNetworkPayload(FARObjectUpdatePayload& Payload)
{
}

void UARTrackedObject::UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, UARCandidateObject* InDetectedObject)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	DetectedObject = InDetectedObject;
	ObjectClassification = EARObjectClassification::SceneObject;
}

void UARTrackedPose::DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds) const
{
	if (TrackedPose.JointTransformSpace != EARJointTransformSpace::Model)
	{
		// TODO: Support joints defined in the parent space
		return;
	}
	
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();
	const FColor OutlineRGB = OutlineColor.ToFColor(false);
	
	#if 1
	{
		// Draw the entire skeleton
		for (int Index = 0; Index < TrackedPose.SkeletonDefinition.NumJoints; ++Index)
		{
			if (!TrackedPose.IsJointTracked[Index])
			{
				continue;
			}
			
			const int ParentIndex = TrackedPose.SkeletonDefinition.ParentIndices[Index];
			if (ParentIndex >= 0 && ParentIndex < TrackedPose.SkeletonDefinition.NumJoints)
			{
				if (TrackedPose.IsJointTracked[ParentIndex])
				{
					const FTransform JointWorldTransform = TrackedPose.JointTransforms[Index] * LocalToWorldTransform;
					const FTransform ParentWorldTransform = TrackedPose.JointTransforms[ParentIndex] * LocalToWorldTransform;
					DrawDebugLine(World, JointWorldTransform.GetLocation(), ParentWorldTransform.GetLocation(), OutlineRGB, false, PersistForSeconds);
				}
			}
		}
	}
	#else
	{
		// Draw individual joint location and name
		for (int Index = 0; Index < TrackedPose.SkeletonDefinition.NumJoints; ++Index)
		{
			if (!TrackedPose.IsJointTracked[Index])
			{
				continue;
			}
			
			const FTransform JointWorldTransform = TrackedPose.JointTransforms[Index] * LocalToWorldTransform;
			DrawDebugPoint(World, JointWorldTransform.GetLocation(), 0.5f, OutlineRGB, false, PersistForSeconds, 0);
			ARDebugHelpers::DrawDebugString(World, JointWorldTransform.GetLocation() + FVector(0.f, 0.f, 10.f),
											TrackedPose.SkeletonDefinition.JointNames[Index].ToString(),
											0.25f * OutlineThickness, OutlineRGB, PersistForSeconds, true);
		}
	}
	#endif
}

void UARTrackedPose::GetNetworkPayload(FARPoseUpdatePayload& Payload)
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();

	Payload.WorldTransform = LocalToWorldTransform;

	Payload.JointTransforms = TrackedPose.JointTransforms;
}

void UARTrackedPose::UpdateTrackedPose(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FARPose3D& InTrackedPose)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	TrackedPose = InTrackedPose;
}

void UARMeshGeometry::GetNetworkPayload(FARMeshUpdatePayload& Payload)
{
	const FTransform LocalToWorldTransform = GetLocalToWorldTransform();

	Payload.WorldTransform = LocalToWorldTransform;
	
	UpdateSessionPayload(Payload.SessionPayload);
}

void UARGeoAnchor::UpdateGeoAnchor(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform,
									  float InLongitude, float InLatitude, float InAltitudeMeters, EARAltitudeSource InAltitudeSource)
{
	Super::UpdateTrackedGeometry(InTrackingSystem, FrameNumber, Timestamp, InLocalToTrackingTransform, InAlignmentTransform);
	
	Longitude = InLongitude;
	Latitude = InLatitude;
	AltitudeMeters = InAltitudeMeters;
	AltitudeSource = InAltitudeSource;
}

void UARGeoAnchor::GetNetworkPayload(FARGeoAnchorUpdatePayload& Payload)
{
	UpdateSessionPayload(Payload.SessionPayload);
	Payload.WorldTransform = GetLocalToWorldTransform();
	Payload.Longitude = Longitude;
	Payload.Latitude = Latitude;
	Payload.AltitudeMeters = AltitudeMeters;
	Payload.AltitudeSource = AltitudeSource;
	Payload.AnchorName = GetName();
}

