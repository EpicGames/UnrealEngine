// Copyright Epic Games, Inc. All Rights Reserved.

#include "ARTrackableNotifyComponent.h"
#include "ARBlueprintLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARTrackableNotifyComponent)

void UARTrackableNotifyComponent::OnRegister()
{
	Super::OnRegister();

	// Add the delegates we want to listen for
	UARBlueprintLibrary::AddOnTrackableAddedDelegate_Handle(FOnTrackableAddedDelegate::CreateUObject(this, &UARTrackableNotifyComponent::OnTrackableAdded));
	UARBlueprintLibrary::AddOnTrackableUpdatedDelegate_Handle(FOnTrackableAddedDelegate::CreateUObject(this, &UARTrackableNotifyComponent::OnTrackableUpdated));
	UARBlueprintLibrary::AddOnTrackableRemovedDelegate_Handle(FOnTrackableAddedDelegate::CreateUObject(this, &UARTrackableNotifyComponent::OnTrackableRemoved));
}

void UARTrackableNotifyComponent::OnUnregister()
{
	// Remove the delegates we are listening for
	UARBlueprintLibrary::ClearOnTrackableAddedDelegates(this);
	UARBlueprintLibrary::ClearOnTrackableUpdatedDelegates(this);
	UARBlueprintLibrary::ClearOnTrackableRemovedDelegates(this);

	Super::OnUnregister();
}

template<typename OBJ_TYPE, typename DELEGATE_TYPE>
bool UARTrackableNotifyComponent::ConditionalDispatchEvent(UARTrackedGeometry* Tracked, DELEGATE_TYPE& Delegate)
{
	if (OBJ_TYPE* CastObject = Cast<OBJ_TYPE>(Tracked))
	{
		if (Delegate.IsBound())
		{
			Delegate.Broadcast(CastObject);
		}
		return true;
	}
	return false;
}

void UARTrackableNotifyComponent::OnTrackableAdded(UARTrackedGeometry* Added)
{
	if (ConditionalDispatchEvent<UARTrackedObject, FTrackableObjectDelegate>(Added, OnAddTrackedObject))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UAREnvironmentCaptureProbe, FTrackableEnvProbeDelegate>(Added, OnAddTrackedEnvProbe))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARFaceGeometry, FTrackableFaceDelegate>(Added, OnAddTrackedFace))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedImage, FTrackableImageDelegate>(Added, OnAddTrackedImage))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedPoint, FTrackablePointDelegate>(Added, OnAddTrackedPoint))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARPlaneGeometry, FTrackablePlaneDelegate>(Added, OnAddTrackedPlane))
	{
		// Do nothing, since we found the right type
	}

	// Always trigger the base version since it matches all trackables
	if (OnAddTrackedGeometry.IsBound())
	{
		OnAddTrackedGeometry.Broadcast(Added);
	}
}

void UARTrackableNotifyComponent::OnTrackableUpdated(UARTrackedGeometry* Updated)
{
	if (ConditionalDispatchEvent<UARTrackedObject, FTrackableObjectDelegate>(Updated, OnUpdateTrackedObject))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UAREnvironmentCaptureProbe, FTrackableEnvProbeDelegate>(Updated, OnUpdateTrackedEnvProbe))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARFaceGeometry, FTrackableFaceDelegate>(Updated, OnUpdateTrackedFace))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedImage, FTrackableImageDelegate>(Updated, OnUpdateTrackedImage))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedPoint, FTrackablePointDelegate>(Updated, OnUpdateTrackedPoint))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARPlaneGeometry, FTrackablePlaneDelegate>(Updated, OnUpdateTrackedPlane))
	{
		// Do nothing, since we found the right type
	}

	// Always trigger the base version since it matches all trackables
	if (OnUpdateTrackedGeometry.IsBound())
	{
		OnUpdateTrackedGeometry.Broadcast(Updated);
	}
}

void UARTrackableNotifyComponent::OnTrackableRemoved(UARTrackedGeometry* Removed)
{
	if (ConditionalDispatchEvent<UARTrackedObject, FTrackableObjectDelegate>(Removed, OnRemoveTrackedObject))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UAREnvironmentCaptureProbe, FTrackableEnvProbeDelegate>(Removed, OnRemoveTrackedEnvProbe))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARFaceGeometry, FTrackableFaceDelegate>(Removed, OnRemoveTrackedFace))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedImage, FTrackableImageDelegate>(Removed, OnRemoveTrackedImage))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARTrackedPoint, FTrackablePointDelegate>(Removed, OnRemoveTrackedPoint))
	{
		// Do nothing, since we found the right type
	}
	else if (ConditionalDispatchEvent<UARPlaneGeometry, FTrackablePlaneDelegate>(Removed, OnRemoveTrackedPlane))
	{
		// Do nothing, since we found the right type
	}

	// Always trigger the base version since it matches all trackables
	if (OnRemoveTrackedGeometry.IsBound())
	{
		OnRemoveTrackedGeometry.Broadcast(Removed);
	}
}

