// Copyright Epic Games, Inc. All Rights Reserved.


#include "ARTypes.h"
#include "HAL/ThreadSafeBool.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARTypes)

bool FARAsyncTask::HadError() const
{
	return bHadError;
}

FString FARAsyncTask::GetErrorString() const
{
	if (bIsDone)
	{
		return Error;
	}
	return FString();
}

bool FARAsyncTask::IsDone() const
{
	return bIsDone;
}

TArray<uint8> FARSaveWorldAsyncTask::GetSavedWorldData()
{
	if (bIsDone)
	{
		return MoveTemp(WorldData);
	}
	return TArray<uint8>();
}

#if WITH_EDITOR
void UARCandidateImage::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property != nullptr)
	{
		float AspectRatio = 1.f;
		if (CandidateTexture != nullptr)
		{
			if (Orientation == EARCandidateImageOrientation::Landscape)
			{
				AspectRatio = (float)CandidateTexture->GetSizeY() / (float)FMath::Max<int32>(1, CandidateTexture->GetSizeX());
			}
			else
			{
				AspectRatio = (float)CandidateTexture->GetSizeX() / (float)FMath::Max<int32>(1, CandidateTexture->GetSizeY());
			}
		}
		// If the texture has changed enforce the aspect ratio on the physical size
		if (PropertyChangedEvent.Property->GetFName() == FName("CandidateTexture"))
		{
			Height = Width * AspectRatio;
		}
		// Adjust width if they changed the height
		else if (PropertyChangedEvent.Property->GetFName() == FName("Height"))
		{
			Width = Height * (1.f / AspectRatio);
		}
		// Adjust height if they changed the width
		else if (PropertyChangedEvent.Property->GetFName() == FName("Width"))
		{
			Height = Width * AspectRatio;
		}
		// Adjust the sizes if they switched the orientation
		else if (PropertyChangedEvent.Property->GetFName() == FName("Orientation"))
		{
			if (Orientation == EARCandidateImageOrientation::Landscape)
			{
				Width = Height * (1.f / AspectRatio);
			}
			else
			{
				Height = Width * AspectRatio;
			}
		}
	}
	
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif


