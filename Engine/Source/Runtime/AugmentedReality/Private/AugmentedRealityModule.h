// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAR, Log, All);
DECLARE_STATS_GROUP(TEXT("AR"), STATGROUP_AR, STATCAT_Advanced);
