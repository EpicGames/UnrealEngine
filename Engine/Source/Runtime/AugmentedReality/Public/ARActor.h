// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARComponent.h"
#include "Engine/GameEngine.h"
#include "ARActor.generated.h"

#define UE_API AUGMENTEDREALITY_API

class UARComponent;

UCLASS(MinimalAPI, BlueprintType, Experimental, Category="AR Gameplay")
class AARActor: public AActor
{
	GENERATED_BODY()
	
public:
	UE_API AARActor();
	
	UFUNCTION(BlueprintCallable, Category="AR Gameplay")
	UE_API UARComponent* AddARComponent(TSubclassOf<UARComponent> InComponentClass, const FGuid& NativeID);
	
	static UE_API void RequestSpawnARActor(FGuid NativeID, UClass* InComponentClass);
	static UE_API void RequestDestroyARActor(AARActor* InActor);
};

USTRUCT()
struct FTrackedGeometryGroup
{
public:
	GENERATED_USTRUCT_BODY()
	
	FTrackedGeometryGroup() = default;
	
	UE_API FTrackedGeometryGroup(UARTrackedGeometry* InTrackedGeometry);
	
	UPROPERTY()
	TObjectPtr<AARActor> ARActor = nullptr;
	
	UPROPERTY()
	TObjectPtr<UARComponent> ARComponent = nullptr;
	
	UPROPERTY()
	TObjectPtr<UARTrackedGeometry> TrackedGeometry = nullptr;
};

#undef UE_API
