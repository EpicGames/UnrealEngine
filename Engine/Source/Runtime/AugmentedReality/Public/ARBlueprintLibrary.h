// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "ARTypes.h"
#include "ARSupportInterface.h"
#include "ARSystem.h"
#include "ARTraceResult.h"
#include "ARSessionConfig.h"
#include "ARTrackable.h"
#include "ARTextures.h"

#include "ARBlueprintLibrary.generated.h"

#define UE_API AUGMENTEDREALITY_API


#define DEFINE_AR_BPLIB_DELEGATE_FUNCS(DelegateName) \
public: \
	static FDelegateHandle Add##DelegateName##Delegate_Handle(const F##DelegateName##Delegate& Delegate) \
	{ \
		auto ARSystem = GetARSystem(); \
		if (ARSystem.IsValid()) \
		{ \
			return ARSystem.Pin()->Add##DelegateName##Delegate_Handle(Delegate); \
		} \
		return Delegate.GetHandle(); \
	} \
	static void Clear##DelegateName##Delegate_Handle(FDelegateHandle& Handle) \
	{ \
		auto ARSystem = GetARSystem(); \
		if (ARSystem.IsValid()) \
		{ \
			ARSystem.Pin()->Clear##DelegateName##Delegate_Handle(Handle); \
			return; \
		} \
		Handle.Reset(); \
	} \
	static void Clear##DelegateName##Delegates(FDelegateUserObject Object) \
	{ \
		auto ARSystem = GetARSystem(); \
		if (ARSystem.IsValid()) \
		{ \
			ARSystem.Pin()->Clear##DelegateName##Delegates(Object); \
		} \
	}

UCLASS(MinimalAPI, meta=(ScriptName="ARLibrary"))
class UARBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Checks if the current device can support AR
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (DisplayName="Is AR Supported", Keywords = "ar augmentedreality augmented reality session start run running"))
	static UE_API bool IsARSupported();
	
	/**
	 * Begin a new Augmented Reality session. Subsequently, use the \c GetARSessionStatus() function to figure out the status of the session.
	 *
	 * @param SessionConfig    Describes the tracking method to use, what kind of geometry to detect in the world, etc.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (DisplayName="Start AR Session", Keywords = "ar augmentedreality augmented reality session start run running"))
	static UE_API void StartARSession(UARSessionConfig* SessionConfig);

	/** Pause a running Augmented Reality session without clearing existing state. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (DisplayName="Pause AR Session", Keywords = "ar augmentedreality augmented reality session stop run running"))
	static UE_API void PauseARSession();

	/** Stop a running Augmented Reality session and clear any state. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (DisplayName="Stop AR Session", Keywords = "ar augmentedreality augmented reality session stop run running"))
	static UE_API void StopARSession();
	
	/**
	 * It is intended that you check the status of the Augmented Reality session on every frame and take action accordingly.
	 * e.g. if the session stopped for an unexpected reason, you might give the user a prompt to re-start the session
	 *
	 * @return The status of a current Augmented Reality session: e.g. Running or Not running for a specific reason.
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Session", meta = (DisplayName="Get AR Session Status", Keywords = "ar augmentedreality augmented reality session start stop run running"))
	static UE_API FARSessionStatus GetARSessionStatus();

	/** @return the configuration that the current session was started with. */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Session", meta = (DisplayName="Get AR Session Config", Keywords = "ar augmentedreality augmented reality session"))
	static UE_API UARSessionConfig* GetSessionConfig();

	/** Starts or stops a battery intensive service on device. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (Keywords = "ar augmentedreality augmented reality capture start stop"))
	static UE_API bool ToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType);

	/** Enable or disable Mixed Reality Capture camera. */
	UFUNCTION(BlueprintCallable, Category = "AR", meta = (Keywords = "ar all"))
	static UE_API void SetEnabledXRCamera(bool bOnOff);

	/** Change screen size of Mixed Reality Capture camera. */
	UFUNCTION(BlueprintCallable, Category = "AR", meta = (Keywords = "ar all"))
	static UE_API FIntPoint ResizeXRCamera(const FIntPoint& InSize);

	
	/**
	 * Set a transform that will be applied to the tracking space. This effectively moves any camera
	 * possessed by the Augmented Reality system such that it is pointing at a different spot
	 * in Unreal's World Space. This is often done to support AR scenarios that rely on static
	 * geometry and/or lighting.
	 *
	 * Note: any movable components that are pinned will appear to stay in place, while anything
	 * not pinned or is not movable (static or stationary) will appear to move.
	 *
	 * \see PinComponent
	 * \see PinComponentToTraceResult
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Alignment", meta = (DisplayName="Set AR Alignment Transform", Keywords = "ar augmentedreality augmented reality tracking alignment"))
	static UE_API void SetAlignmentTransform( const FTransform& InAlignmentTransform );
	
	
	/**
	 * Perform a line trace against any real-world geometry as tracked by the AR system.
	 *
	 * @param ScreenCoord	         Coordinate of the point on the screen from which to cast a ray into the tracking space.
	 *
	 * @return a list of \c FARTraceResult sorted by distance from camera.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Trace Result", meta = (AdvancedDisplay="1", Keywords = "ar augmentedreality augmented reality tracking tracing linetrace"))
	static UE_API TArray<FARTraceResult> LineTraceTrackedObjects( const FVector2D ScreenCoord, bool bTestFeaturePoints = true, bool bTestGroundPlane = true, bool bTestPlaneExtents = true, bool bTestPlaneBoundaryPolygon = true );
	
	/**
	 * Perform a line trace against any real-world geometry as tracked by the AR system.
	 *
	 * @param Start					Start point of the trace, in world space.
	 * @param End					End point of the trace, in world space.
	 *
	 * @return a list of \c FARTraceResult sorted by distance from camera.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Trace Result", meta = (AdvancedDisplay = "1", Keywords = "ar augmentedreality augmented reality tracking tracing linetrace"))
	static UE_API TArray<FARTraceResult> LineTraceTrackedObjects3D(const FVector Start, const FVector End, bool bTestFeaturePoints = true, bool bTestGroundPlane = true, bool bTestPlaneExtents = true, bool bTestPlaneBoundaryPolygon = true);
	
	/** @return how well the tracking system is performing at the moment */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get AR Tracking Quality", Keywords = "ar augmentedreality augmented reality tracking quality"))
	static UE_API EARTrackingQuality GetTrackingQuality();
	
	/** @return The reason for the current limited tracking state */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get AR Tracking Quality Reason", Keywords = "ar augmentedreality augmented reality tracking quality reason"))
	static UE_API EARTrackingQualityReason GetTrackingQualityReason();
	
	/** @return a list of all the real-world geometry as currently seen by the Augmented Reality system */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get All AR Geometries", Keywords = "ar augmentedreality augmented reality tracking geometry anchor"))
	static UE_API TArray<UARTrackedGeometry*> GetAllGeometries();
	
	/** @return a list of all the real-world geometry of the specified class as currently seen by the Augmented Reality system */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get All AR Geometries By Class", Keywords = "ar augmentedreality augmented reality tracking geometry anchor", DeterminesOutputType = "GeometryClass"))
	static UE_API TArray<UARTrackedGeometry*> GetAllGeometriesByClass(TSubclassOf<UARTrackedGeometry> GeometryClass);
	
	/** @return the AR texture for the specified type */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Texture", meta = (DisplayName="Get AR Texture", Keywords = "ar augmentedreality augmented reality texture"))
	static UE_API UARTexture* GetARTexture(EARTextureType TextureType);
	
	/**
	 * Test whether this type of session is supported by the current Augmented Reality platform.
	 * e.g. is your device capable of doing positional tracking or orientation only?
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Is AR Session Type Supported", Keywords = "ar augmentedreality augmented reality tracking"))
	static UE_API bool IsSessionTypeSupported(EARSessionType SessionType);
	
	
	/** Given some real-world geometry being tracked by the Augmented Reality system, draw it on the screen for debugging purposes (rudimentary)*/
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Debug", meta = (WorldContext = "WorldContextObject", AdvancedDisplay="1", Keywords = "ar augmentedreality augmented reality tracked geometry debug draw"))
	static UE_API void DebugDrawTrackedGeometry( UARTrackedGeometry* TrackedGeometry, UObject* WorldContextObject, FLinearColor Color = FLinearColor(1.0f, 1.0f, 0.0f, 0.75f), float OutlineThickness=5.0f, float PersistForSeconds = 0.0f );
	
	/** Given a \c UARPin, draw it for debugging purposes. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Debug", meta = (WorldContext = "WorldContextObject", AdvancedDisplay="1", Keywords = "ar augmentedreality augmented reality pin debug draw"))
	static UE_API void DebugDrawPin( UARPin* ARPin, UObject* WorldContextObject, FLinearColor Color = FLinearColor(1.0f, 1.0f, 0.0f, 0.75f), float Scale=5.0f, float PersistForSeconds = 0.0f );
	
	
	/**
	 * An AugmentedReality session can be configured to provide light estimates.
	 * The specific approach to light estimation can be configured by the \c UARSessionConfig
	 * specified during \c StartARSession(). This function assumes that you will cast
	 * the returned \c UARLightEstimate to a derived type corresponding to your
	 * session config.
	 *
	 * @return a \c UARLighEstimate that can be cast to a derived class.
	 */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Light Estimate" )
	static UE_API UARLightEstimate* GetCurrentLightEstimate();
	
	
	/**
	 * Pin an Unreal Component to a location in tracking spce (i.e. the real world).
	 *
	 * @param ComponentToPin         The component that should be pinned.
	 * @param PinToWorldTransform    A transform (in Unreal World Space) that corresponds to
	 *                               a physical location where the component should be pinned.
	 * @param TrackedGeometry        An optional, real-world geometry that is recognized by the
	 *                               AR system; any correction to the position of this geometry
	 *                               will be applied to the pinned component.
	 * @param DebugName              An optional name that will be displayed when this
	 *                               pin is being drawn for debugging purposes.
	 *
	 * @return an object representing the pin that connects \c ComponentToPin component to a real-world
	 *         location and optionally to the \c TrackedGeometry.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (AdvancedDisplay="3", Keywords = "ar augmentedreality augmented reality tracking arpin tracked geometry pinning anchor"))
	static UE_API UARPin* PinComponent( USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None );
	
	/**
	 * A convenient version of \c PinComponent() that can be used in conjunction
	 * with a result of a \c LineTraceTrackedObjects call.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (AdvancedDisplay="2", Keywords = "ar augmentedreality augmented reality tracking arpin tracked geometry pinning anchor"))
	static UE_API UARPin* PinComponentToTraceResult( USceneComponent* ComponentToPin, const FARTraceResult& TraceResult, const FName DebugName = NAME_None );
	
	/**
	 * Associate a component with an ARPin, so that its transform will be updated by the pin.  Any previously associated component will be detached.
	 *
	 * @param ComponentToPin	The Component which will be updated by the Pin.
	 * @param Pin				The Pin which the component will be updated by.
	 *
	 * @return					True if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin tracked geometry pinning anchor"))
	static UE_API bool PinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin);

	/** Given a pinned \c ComponentToUnpin, remove its attachment to the real world. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin tracked geometry pinning anchor"))
	static UE_API void UnpinComponent( USceneComponent* ComponentToUnpin );
	
	/** Remove a pin such that it no longer updates the associated component. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin tracked geometry pinning anchor"))
	static UE_API void RemovePin( UARPin* PinToRemove );
	
	/** Get a list of all the \c UARPin objects that the Augmented Reality session is currently using to connect virtual objects to real-world, tracked locations. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor"))
	static UE_API TArray<UARPin*> GetAllPins();

	/**
	 * Is ARPin Local Store Supported
	 *
	 * @return					True if Local Pin saving is supported by the device/platform.
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API bool IsARPinLocalStoreSupported();

	/**
	 * Is ARPin Local Store Ready
	 *
	 * @return					True if local store is ready for use.
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API bool IsARPinLocalStoreReady();

	/**
	 * Load all ARPins from local save
	 * Note: Multiple loads of a saved pin may result in duplicate pins OR overwritten pins.  It is reccomended to only load once.
	 *
	 * @return					Map of SaveName:ARPin.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API TMap<FName, UARPin*> LoadARPinsFromLocalStore();
	
	/**
	 * Save an ARPin to local store
	 * @param InName			The save name for the pin.
	 * @param InPin				The ARPin which will be saved to the local store.
	 *
	 * @return					True if saved successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API bool SaveARPinToLocalStore(FName InSaveName, UARPin* InPin);
	
	/**
	 * Remove an ARPin from the local store
	 * @param InName			The save name to remove.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API void RemoveARPinFromLocalStore(FName InSaveName);

	/**
	 * Remove all ARPins from the local store
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|ARPin", meta = (Keywords = "ar augmentedreality augmented reality tracking arpin anchor LocalStore"))
	static UE_API void RemoveAllARPinsFromLocalStore();

	/** Adds an environment capture probe to the ar world */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Add AR Environment Probe", Keywords = "ar augmentedreality augmented reality tracking anchor"))
	static UE_API bool AddManualEnvironmentCaptureProbe(FVector Location, FVector Extent);

	/** @return the current world mapping status for the AR world */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get AR World Mapping Status", Keywords = "ar augmentedreality augmented reality tracking anchor"))
	static UE_API EARWorldMappingState GetWorldMappingStatus();
	
	/** @return the raw point cloud view of the AR scene */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Tracking", meta = (DisplayName="Get AR Point Cloud", Keywords = "ar augmentedreality augmented reality tracking point cloud"))
	static UE_API TArray<FVector> GetPointCloud();

	/** @return The list of supported video formats for this device */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Capabilities", meta = (DisplayName="Get Supported AR Video Formats", Keywords = "ar augmentedreality augmented reality config video formats"))
	static UE_API TArray<FARVideoFormat> GetSupportedVideoFormats(EARSessionType SessionType);

	static UE_API TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> SaveWorld();
	static UE_API TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> GetCandidateObject(FVector Location, FVector Extent);
	
	/**
	 * Create an ARCandidateImage object and add it to the ARCandidateImageList of the given \c UARSessionConfig object.
	 *
	 * Note that you need to restart the AR session with the \c UARSessionConfig you are adding to to make the change take effect.
	 *
	 * On ARCore platform, you can leave the PhysicalWidth to 0 if you don't know the physical size of the image or
	 * the physical size is dynamic. And this function takes time to perform non-trivial image processing (20ms - 30ms),
	 * and should be run on a background thread.
	 *
	 * @return A \c UARCandidateImage Object pointer if the underlying ARPlatform added the candidate image at runtime successfully.
	 *		  Return nullptr otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Session", meta = (Keywords = "ar augmentedreality augmented reality candidate image"))
	static UE_API UARCandidateImage* AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth);

	/** @return if a particular session feature is supported with the specified session type on the current platform */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Session", meta = (DisplayName="Is AR Session Tracking Feature Supported", Keywords = "ar augmentedreality augmented reality session tracking feature"))
	static UE_API bool IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature);
	
	/** @return if a particular scene reconstruction method is supported with the specified session type on the current platform */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Session", meta = (DisplayName="Is AR Scene Reconstruction Supported", Keywords = "ar augmentedreality augmented reality session scene reconstruction"))
	static UE_API bool IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod);
	
	/** @return all the 2D poses tracked by the AR system */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Pose Tracking", meta = (DisplayName="Get All AR Tracked 2D Poses", Keywords = "ar augmentedreality augmented reality pose tracking"))
	static UE_API TArray<FARPose2D> GetAllTracked2DPoses();
	
	/**
	 * Try to determine the classification of the object at a world space location
	 * @InWorldLocation: the world location where the classification is needed
	 * @OutClassification: the classification result
	 * @OutClassificationLocation: the world location at where the classification is calculated
	 * @MaxLocationDiff: the max distance between the specified world location and the classification location
	 * @return: whether a valid classification result is calculated
	*/
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Classification")
	static UE_API bool GetObjectClassificationAtLocation(const FVector& InWorldLocation, EARObjectClassification& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff = 10.f);
	
	/**
	 * For a point P in the AR local space, whose location and rotation are "OriginLocation" and "OriginRotation" in the world space
	 * modify the alignment transform so that the same point P will be transformed to the origin in the world space.
	 * @bIsTransformInWorldSpace: whether "OriginLocation" and "OriginRotation" are specified in UE4's world space or AR system's local space.
	 * @bMaintainUpDirection: if set, only the yaw roation of the alignment transform will be modified, pitch and roll will be zeroed out.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Alignment", meta = (DisplayName="Set AR World Origin Location and Rotation", Keywords = "ar augmentedreality augmented reality world origin"))
	static UE_API void SetARWorldOriginLocationAndRotation(FVector OriginLocation, FRotator OriginRotation, bool bIsTransformInWorldSpace = true, bool bMaintainUpDirection = true);
	
	/**
	 * Helper function that modifies the alignment transform scale so that virtual content in the world space appears to be "scaled".
	 * Note that ultimately the scaling effect is achieved through modifying the translation of the camera:
	 * moving the camera further away from the origin makes objects appear to be smaller, and vice versa.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Alignment", meta = (DisplayName="Set AR World Scale", Keywords = "ar augmentedreality augmented reality world scale"))
	static UE_API void SetARWorldScale(float InWorldScale);
	
	/** @return the AR world scale, see "SetARWorldScale" */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Alignment", meta = (DisplayName="Get AR World Scale", Keywords = "ar augmentedreality augmented reality world scale"))
	static UE_API float GetARWorldScale();
	
	/** @return the alignment transform, see "SetAlignmentTransform" */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Alignment", meta = (DisplayName="Get Alignment Transform", Keywords = "ar augmentedreality augmented reality alignment transform"))
	static UE_API FTransform GetAlignmentTransform();
	
	/**
	 * Manually add a tracked point with name and world transform.
	 * @WorldTransform: transform in the world space where the point should be created.
	 * @PointName: the name of the created point, must be non-empty.
	 * @bDeletePointsWithSameName: if existing points with the same name should be deleted.
	 * @return if the operation succeeds.
	 * Note that this is an async operation - the added point won't be available until a few frames later.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality", meta = (Keywords = "ar augmentedreality augmented reality add point"))
	static UE_API bool AddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName = true);
	
	/** @return a list of the tracked points with the given name */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality", meta = (Keywords = "ar augmentedreality augmented reality find point"))
	static UE_API TArray<UARTrackedPoint*> FindTrackedPointsByName(const FString& PointName);
	
	// Static helpers to create the methods needed to add/remove delegates from the AR system
	DEFINE_AR_BPLIB_DELEGATE_FUNCS(OnTrackableAdded)
	DEFINE_AR_BPLIB_DELEGATE_FUNCS(OnTrackableUpdated)
	DEFINE_AR_BPLIB_DELEGATE_FUNCS(OnTrackableRemoved)
	// End helpers

	//Alignment helpers
	UFUNCTION(BlueprintCallable, Category = Alignment)
	static UE_API void CalculateClosestIntersection(const TArray<FVector>& StartPoints, const TArray<FVector>& EndPoints, FVector& ClosestIntersection);

	// Computes a transform that aligns two coordinate systems. Requires the transform of the same known point in each coordinate system.
	UFUNCTION(BlueprintCallable, Category = Alignment)
	static UE_API void CalculateAlignmentTransform(const FTransform& TransformInFirstCoordinateSystem, const FTransform& TransformInSecondCoordinateSystem, FTransform& AlignmentTransform);
	
	/** @return the max number of faces can be tracked at the same time */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Face Tracking")
	static UE_API int32 GetNumberOfTrackedFacesSupported();
	
	/** @return the intrinsics of the AR camera. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	static UE_API bool GetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics);
	
	/** @return a list of all the real-world geometry of the specified class as currently seen by the Augmented Reality system */
	template<class T>
	static TArray<T*> GetAllGeometriesByClass()
	{
		TArray<T*> Geometries;
		for (auto Geometry : GetAllGeometries())
		{
			if (auto CastedGeometry = Cast<T>(Geometry))
			{
				Geometries.Add(CastedGeometry);
			}
		}
		return MoveTemp(Geometries);
	}
	
public:
	static UE_API void RegisterAsARSystem(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& NewArSystem);
	
private:
	static UE_API const TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe>& GetARSystem();
	static UE_API TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> RegisteredARSystem;
};


UCLASS(MinimalAPI, meta=(ScriptName="ARTraceResultLibrary"))
class UARTraceResultLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
	public:
	
	/** @return  the distance from the camera to the traced location in Unreal Units. */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API float GetDistanceFromCamera( const FARTraceResult& TraceResult );
	
	/**
	 * @return The transform of the trace result in tracking space (after it is modified by the \c AlignmentTransform).
	 *
	 * \see SetAlignmentTransform()
	 */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API FTransform GetLocalToTrackingTransform( const FARTraceResult& TraceResult );
	
	/** @return Get the transform of the trace result in Unreal World Space. */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API FTransform GetLocalToWorldTransform( const FARTraceResult& TraceResult );
	
	/** @return Get the transform of the trace result in the AR system's local space. */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API FTransform GetLocalTransform( const FARTraceResult& TraceResult );
	
	/** @return Get the real-world object (as observed by the Augmented Reality system) that was intersected by the line trace. */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API UARTrackedGeometry* GetTrackedGeometry( const FARTraceResult& TraceResult );
	
	/** @return  Get the type of the tracked object (if any) that effected this trace result. */
	UFUNCTION( BlueprintPure, Category = "AR AugmentedReality|Trace Result" )
	static UE_API EARLineTraceChannels GetTraceChannel( const FARTraceResult& TraceResult );
};

#undef UE_API
