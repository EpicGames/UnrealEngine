// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "Tickable.h"
#include "Kismet/BlueprintAsyncActionBase.h"

#include "ARTypes.h"
#include "ARSupportInterface.h"

#include "ARBlueprintProxy.generated.h"

#define UE_API AUGMENTEDREALITY_API

UCLASS(MinimalAPI, Abstract)
class UARBaseAsyncTaskBlueprintProxy :
	public UBlueprintAsyncActionBase,
	public FTickableGameObject
{
	GENERATED_UCLASS_BODY()
	
public:
	//~ Begin FTickableObject Interface
	UE_API virtual void Tick(float DeltaTime) override;
	virtual bool IsTickable() const override { return !HasAnyFlags(RF_ClassDefaultObject) && bShouldTick; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UARBaseAsyncTaskBlueprintProxy, STATGROUP_Tickables); }
	//~ End FTickableObject Interface

	virtual void ReportSuccess() { check(0); }
	virtual void ReportFailure() { check(0); }

	static UE_API void RegisterAsARSystem(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& NewArSystem);

protected:
	static UE_API const TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe>& GetARSystem();
	/** The async task to check during Tick() */
	TSharedPtr<FARAsyncTask, ESPMode::ThreadSafe> AsyncTask;
	
private:
	/** True until the async task completes, then false */
	bool bShouldTick;

	static UE_API TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> RegisteredARSystem;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FARSaveWorldPin, const TArray<uint8>&, SavedWorld);

UCLASS()
class UARSaveWorldAsyncTaskBlueprintProxy :
	public UARBaseAsyncTaskBlueprintProxy
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FARSaveWorldPin OnSuccess;
	
	UPROPERTY(BlueprintAssignable)
	FARSaveWorldPin OnFailed;
	
	/**
	 * Saves an AR world to a byte array for network replication or saving to disk
	 *
	 * @param bCompressData whether to compress the data or not
	 */
	UFUNCTION(BlueprintCallable, meta=(DisplayName="Save AR World", BlueprintInternalUseOnly="true", Category = "Augmented Reality", WorldContext = "WorldContextObject"))
	static UARSaveWorldAsyncTaskBlueprintProxy* ARSaveWorld(UObject* WorldContextObject);

private:
	// UBlueprintAsyncActionBase interface
	virtual void Activate() override;
	//~UBlueprintAsyncActionBase interface
	
	/** The async task to check during Tick() */
	TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> SaveWorldTask;
	
	virtual void ReportSuccess() override;
	virtual void ReportFailure() override;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FARGetCandidateObjectPin, UARCandidateObject*, SavedObject);

UCLASS()
class UARGetCandidateObjectAsyncTaskBlueprintProxy :
	public UARBaseAsyncTaskBlueprintProxy
{
	GENERATED_BODY()

public:
	UPROPERTY(BlueprintAssignable)
	FARGetCandidateObjectPin OnSuccess;
	
	UPROPERTY(BlueprintAssignable)
	FARGetCandidateObjectPin OnFailed;

	/**
	 * Saves the point cloud centered at the specified location capturing all of the features within the specified extent as an object that can be detected later
	 *
	 * @param Location the center of the extent to grab features at
	 * @param Extent the size of the region to grab feature points
	 */
	UFUNCTION(BlueprintCallable, Meta=(DisplayName="Get AR Candidate Object", BlueprintInternalUseOnly = "true", Category = "Augmented Reality", WorldContext = "WorldContextObject"))
	static UARGetCandidateObjectAsyncTaskBlueprintProxy* ARGetCandidateObject(UObject* WorldContextObject, FVector Location, FVector Extent);
	
	FVector Location;
	FVector Extent;
	
private:
	// UBlueprintAsyncActionBase interface
	virtual void Activate() override;
	//~UBlueprintAsyncActionBase interface
	
	/** The async task to check during Tick() */
	TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> CandidateObjectTask;
	
	virtual void ReportSuccess() override;
	virtual void ReportFailure() override;
};

#undef UE_API
