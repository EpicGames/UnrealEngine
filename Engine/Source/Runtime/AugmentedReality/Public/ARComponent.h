// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "Components/SceneComponent.h"
#include "PackedNormal.h"
#include "MRMeshBufferDefines.h"
#include "ARComponent.generated.h"

#define UE_API AUGMENTEDREALITY_API

class USceneComponent;
class UMRMeshComponent;
class UARSessionConfig;
class UARTrackedGeometry;
class UMaterialInterface;

UENUM(BlueprintType)
enum class EARSessionConfigFlags : uint8
{
	None = 0,
	GenerateMeshData = 1 << 0,
	RenderMeshDataInWireframe = 1 << 1,
	GenerateCollisionForMeshData = 1 << 2,
	GenerateNavMeshForMeshData = 1 << 3,
	UseMeshDataForOcclusion = 1 << 4,
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARSessionPayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	int32 ConfigFlags = 0;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	TObjectPtr<UMaterialInterface> DefaultMeshMaterial = nullptr;

	UPROPERTY(BlueprintReadOnly, Category = "AR")
	TObjectPtr<UMaterialInterface> DefaultWireframeMeshMaterial = nullptr;

	void SetFlag(EARSessionConfigFlags InFlag);
	bool HasFlag(EARSessionConfigFlags InFlag) const;
	void FromSessionConfig(const UARSessionConfig& InConfig);
	bool ShouldCreateMeshComponent() const;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARPlaneUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector Center = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector Extents = FVector::ZeroVector;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	TArray<FVector> BoundaryVertices;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	EARObjectClassification ObjectClassification = EARObjectClassification::Unknown;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARPointUpdatePayload
{
	GENERATED_BODY()
};


USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARFaceUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector LeftEyePosition = FVector(0.0f);
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector RightEyePosition = FVector(0.0f);

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector LookAtTarget = FVector(0.0f);
};


USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARImageUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	TObjectPtr<UARCandidateImage> DetectedImage = nullptr;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FVector2D EstimatedSize = FVector2D::ZeroVector;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARQRCodeUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FVector Extents = FVector(0.0f);

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FString QRCode;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARPoseUpdatePayload
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	TArray<FTransform> JointTransforms;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FAREnvironmentProbeUpdatePayload
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
};

USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARObjectUpdatePayload
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
};


USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARMeshUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	EARObjectClassification ObjectClassification = EARObjectClassification::Unknown;
};


USTRUCT(BlueprintType, Category = "AR Gameplay")
struct FARGeoAnchorUpdatePayload
{
	GENERATED_BODY()
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FARSessionPayload SessionPayload;
	
	UPROPERTY(BlueprintReadWrite, Category = "AR")
	FTransform WorldTransform = FTransform::Identity;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	float Longitude = 0.f;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	float Latitude = 0.f;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	float AltitudeMeters = 0.f;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	EARAltitudeSource AltitudeSource = EARAltitudeSource::Unknown;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR")
	FString AnchorName;
};

struct FAccumulatedNormal
{
	FVector Normal = FVector::ZeroVector;
	uint32 NumFaces = 0;
	
	template<typename VertexType>
	static void CalculateVertexNormals(TArray<FAccumulatedNormal>& AccumulatedNormals, const TArray<VertexType>& Vertices, const TArray<MRMESH_INDEX_TYPE>& Indices, TArray<FPackedNormal>& OutTangentData, FVector MeshCenter = FVector::ZeroVector, float PositionScale = 1.f);
};

/**
 * ARComponent handles replication and visualization update for AR tracked geometries
 * ARComponent is created in either multiplayer or local only environment
 *
 * To customize the visualization logic, override 'UpdateVisualization' in Blueprint
 * or 'UpdateVisualization_Implementation' in C++
 *
 * To customize the replication logic, disable 'bUseDefaultReplication' and implement
 * your own replication logic when the ARComponent is updated (see how 'Update' is implemented in the sub classes)
 *
 * A MRMeshComponent will be created if mesh visualization is enabled (see UARSessionConfig::bGenerateMeshDataFromTrackedGeometry)
 * It can be accessed via 'GetMRMesh' or 'UARTrackedGeometry::GetUnderlyingMesh'
 */
UCLASS(MinimalAPI, abstract, BlueprintType, Category = "AR Gameplay")
class UARComponent : public USceneComponent
{
	GENERATED_BODY()

	UE_API virtual void BeginPlay() override;

public:
	UE_API UARComponent();
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay")
	UE_API void SetNativeID(FGuid NativeID);
	
	UFUNCTION(BlueprintPure, Category = "AR Gameplay")
	UMRMeshComponent* GetMRMesh() { return MRMeshComponent; }
	
	/** Event when native representation is removed, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Remove"))
	UE_API void ReceiveRemove();
	
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, meta = (DisplayName = "Update Visualization"), Category = "AR Gameplay")
	UE_API void UpdateVisualization();
	
	virtual void Update(UARTrackedGeometry* TrackedGeometry) {};
	UE_API virtual void Remove(UARTrackedGeometry* TrackedGeometry);

	UPROPERTY(Replicated)
	FGuid NativeID;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FMRMeshDelegate, UMRMeshComponent*);
	FMRMeshDelegate OnMRMeshCreated;
	FMRMeshDelegate OnMRMeshDestroyed;
	
protected:
	UE_API virtual void OnUnregister() override;
	
	UFUNCTION()
	UE_API virtual void OnRep_Payload();
	
	UE_API void RemoveMeshComponent(UARTrackedGeometry* TrackedGeometry);
	
	UE_API void ManageMeshComponentForDebugMode(bool bDebugModeEnabled, const FARSessionPayload& SessionPayload);
	
	UE_API FLinearColor GetDebugColor() const;
	
	/** If the default replication logic should be used for this component */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	bool bUseDefaultReplication = true;
	
	/**
	 * The default material to be used for the generated mesh component.
	 * If not set, the DefaultMeshMaterial from ARSessionConfig will be used.
	 */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TObjectPtr<UMaterialInterface> DefaultMeshMaterial = nullptr;

	/**
	 * The default wireframe material to be used for the generated mesh component.
	 * If not set, the DefaultMeshMaterial from ARSessionConfig will be used.
	 */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TObjectPtr<UMaterialInterface> DefaultWireframeMeshMaterial = nullptr;

	UPROPERTY()
	TObjectPtr<UMRMeshComponent> MRMeshComponent;
	
	/** The tracked geometry used for updating this component, only set on "local" client */
	UPROPERTY()
	TObjectPtr<UARTrackedGeometry> MyTrackedGeometry;
	
	/** If this component has ever been updated */
	bool bFirstUpdate = true;
	
	/** If 'ReceiveRemove' has been called for this component */
	bool bIsRemoved = false;
	
	bool bInDebugMode = false;
	bool bSavedWireframeMode;
	FLinearColor SavedWireframeColor;
};

UENUM(BlueprintType)
enum class EPlaneComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Use different coloration to indicate if the plane belongs to the local or remote client */
	ShowNetworkRole = 1,
	
	/** Use different coloration to indicate the classification of this plane */
	ShowClassification = 2,
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARPlaneComponent : public UARComponent
{
	GENERATED_BODY()

public:
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARPlaneUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARPlaneUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetPlaneComponentDebugMode(EPlaneComponentDebugMode NewDebugMode);
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetObjectClassificationDebugColors(const TMap<EARObjectClassification, FLinearColor>& InColors);
	
	UFUNCTION(BlueprintPure, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API const TMap<EARObjectClassification, FLinearColor>& GetObjectClassificationDebugColors();
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARPlaneUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARPlaneUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	UE_API virtual void UpdateVisualization_Implementation() override;
	static UE_API int32 GetDebugMode();
};


UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARPointComponent : public UARComponent
{
	GENERATED_BODY()
public:
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARPointUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARPointUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARPointUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARPointUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
};

UENUM(BlueprintType)
enum class EFaceComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Display vectors for both eyes */
	ShowEyeVectors = 1,
	
	/** Display the face mesh in wireframe */
	ShowFaceMesh = 2,
};

UENUM(BlueprintType)
enum class EARFaceTransformMixing : uint8
{
	/** Uses the component's transform exclusively. Only setting for non-tracked meshes */
	ComponentOnly,
	
	/** Use the component's location and apply the rotation from the tracked mesh */
	ComponentLocationTrackedRotation,
	
	/** Concatenate the component and the tracked face transforms */
	ComponentWithTracked,
	
	/** Use only the tracked face transform */
	TrackingOnly
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARFaceComponent : public UARComponent
{
	GENERATED_BODY()

public:
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARFaceUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARFaceUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetFaceComponentDebugMode(EFaceComponentDebugMode NewDebugMode);
	
protected:
	/**	Determines how the transform from tracking data and the component's transform are mixed together. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AR Gameplay")
	EARFaceTransformMixing TransformSetting = EARFaceTransformMixing::TrackingOnly;
	
	/**	Whether to automatically update the vertex normal when the mesh is updated. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AR Gameplay")
	bool bUpdateVertexNormal = false;
	
	/**	Whether the mesh should be rotated so that it's facing out of the screen. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AR Gameplay")
	bool bFaceOutOfScreen = true;
	
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARFaceUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARFaceUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	UE_API virtual void UpdateVisualization_Implementation() override;
	static UE_API int32 GetDebugMode();
	
private:
	/** Temporary data used for calculating the normals from vertices */
	TArray<FAccumulatedNormal> AccumulatedNormals;
	
	TArray<FPackedNormal> TangentData;
};

UENUM(BlueprintType)
enum class EImageComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Display info about the detected image */
	ShowDetectedImage = 1,
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARImageComponent : public UARComponent
{
	GENERATED_BODY()
	
public:
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARImageUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARImageUpdatePayload& Payload);
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetImageComponentDebugMode(EImageComponentDebugMode NewDebugMode);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARImageUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARImageUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	UE_API virtual void UpdateVisualization_Implementation() override;
	static UE_API int32 GetDebugMode();
};

UENUM(BlueprintType)
enum class EQRCodeComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Show info about the detected QR code */
	ShowQRCode = 1,
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARQRCodeComponent : public UARComponent
{
	GENERATED_BODY()

public:
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARQRCodeUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARQRCodeUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetQRCodeComponentDebugMode(EQRCodeComponentDebugMode NewDebugMode);
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARQRCodeUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARQRCodeUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	UE_API virtual void UpdateVisualization_Implementation() override;
	static UE_API int32 GetDebugMode();
};

UENUM(BlueprintType)
enum class EPoseComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Show the skeleton with debug draw */
	ShowSkeleton = 1,
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARPoseComponent : public UARComponent
{
	GENERATED_BODY()

public:
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARPoseUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARPoseUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetPoseComponentDebugMode(EPoseComponentDebugMode NewDebugMode);
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARPoseUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARPoseUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	static UE_API int32 GetDebugMode();
};


UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UAREnvironmentProbeComponent : public UARComponent
{
	GENERATED_BODY()
public:
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FAREnvironmentProbeUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FAREnvironmentProbeUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FAREnvironmentProbeUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FAREnvironmentProbeUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
};


UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARObjectComponent : public UARComponent
{
	GENERATED_BODY()
public:
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARObjectUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARObjectUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARObjectUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARObjectUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
};


UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARMeshComponent : public UARComponent
{
	GENERATED_BODY()
public:
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARMeshUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARMeshUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARMeshUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARMeshUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	UE_API virtual void UpdateVisualization_Implementation() override;
};


UENUM(BlueprintType)
enum class EGeoAnchorComponentDebugMode : uint8
{
	/** The debug mode is disabled */
	None = 0,
	
	/** Display Geo related data */
	ShowGeoData = 1,
};

UCLASS(MinimalAPI, Blueprintable, BlueprintType, Category = "AR Gameplay")
class UARGeoAnchorComponent : public UARComponent
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "AR Gameplay | Debug", meta=(DevelopmentOnly))
	static UE_API void SetGeoAnchorComponentDebugMode(EGeoAnchorComponentDebugMode NewDebugMode);
	
	/** Event when native representation is first added, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Add"))
	UE_API void ReceiveAdd(const FARGeoAnchorUpdatePayload& Payload);
	
	/** Event when native representation is updated, called on server and clients. */
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Update"))
	UE_API void ReceiveUpdate(const FARGeoAnchorUpdatePayload& Payload);
	
	/** Overridable native event for when native representation is updated. */
	UE_API virtual void Update(UARTrackedGeometry* TrackedGeometry) override;
	
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	
protected:
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerUpdatePayload(const FARGeoAnchorUpdatePayload& NewPayload);
	
	UPROPERTY(BlueprintReadOnly, ReplicatedUsing=OnRep_Payload, Category = "AR Gameplay")
	FARGeoAnchorUpdatePayload ReplicatedPayload;
	
	UE_API virtual void OnRep_Payload() override;
	static UE_API int32 GetDebugMode();
};

#undef UE_API
