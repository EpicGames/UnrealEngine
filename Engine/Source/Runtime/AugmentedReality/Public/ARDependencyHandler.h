// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Engine/LatentActionManager.h"
#include "Features/IModularFeature.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ARDependencyHandler.generated.h"

#define UE_API AUGMENTEDREALITY_API

class UARSessionConfig;
struct FFrame;

UENUM(BlueprintType)
enum class EARServiceAvailability : uint8
{
	/** Availability is unknown due to an error during checking */
	UnknownError,
	
	/** Availability is being checked */
	UnknownChecking,
	
	/** Availability is unknown due to timeout during checking */
	UnknownTimedOut,
	
	/** The device is not capable of running the AR service */
	UnsupportedDeviceNotCapable,
	
	/** AR service is not installed */
	SupportedNotInstalled,
	
	/** AR service is installed but the version is too old */
	SupportedVersionTooOld,
	
	/** AR service is supported and installed */
	SupportedInstalled,
};

UENUM(BlueprintType)
enum class EARServiceInstallRequestResult : uint8
{
	/** AR service is installed */
	Installed,
	
	/** The device is not capable of running the AR service */
	DeviceNotCompatible,
	
	/** The user declined the request to install the AR service */
	UserDeclinedInstallation,
	
	/** Other error while installing the AR service */
	FatalError,
};

UENUM(BlueprintType)
enum class EARServicePermissionRequestResult : uint8
{
	/** The permission is granted by the user */
	Granted,
	
	/** The permission is denied by the user */
	Denied,
};

/**
 * Helper class that allows the user to explicitly request AR service installation and permission granting.
 * Recommended flow for explicit management:
 * 1. Call "GetARDependencyHandler" to get a handler, if valid:
 * 2. Call "CheckARServiceAvailability" to check availability, if the device is supported:
 * 3. Call "InstallARService" to install AR service dependency, if installed:
 * 4. Call "RequestARSessionPermission" to request permission, if granted:
 * 5. Call "UARBlueprintLibrary::StartARSession" to start the session.
 * Alternatively, you can also call "StartARSessionLatent" which handles dependency and permission internally.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, Category="")
class UARDependencyHandler : public UObject, public IModularFeature
{
	GENERATED_BODY()
	
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("ARDependencyHandler");
		return FeatureName;
	}
	
	/**
	 * @return the dependency handler for the current platform.
	 * Can return null if the current platform doesn't support AR, or the AR system doesn't require dependency handling.
	 */
	UFUNCTION(BlueprintCallable, Category = "")
	static UE_API UARDependencyHandler* GetARDependencyHandler();
	
	/** Latent action to check AR availability on the current platform. */
	UFUNCTION(BlueprintCallable, Category = "", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	virtual void CheckARServiceAvailability(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceAvailability& OutAvailability) PURE_VIRTUAL(UARDependencyHandler::CheckARServiceAvailability, );
	
	/** Latent action to install AR service on the current platform. */
	UFUNCTION(BlueprintCallable, Category = "", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	virtual void InstallARService(UObject* WorldContextObject, FLatentActionInfo LatentInfo, EARServiceInstallRequestResult& OutInstallResult) PURE_VIRTUAL(UARDependencyHandler::InstallARService, );
	
	/** Latent action to request permission to run the supplied session configuration. */
	UFUNCTION(BlueprintCallable, Category = "", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	virtual void RequestARSessionPermission(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo, EARServicePermissionRequestResult& OutPermissionResult) PURE_VIRTUAL(UARDependencyHandler::RequestARSessionPermission, );
	
	/**
	 * Latent action to start AR session.
	 * Will make sure dependency and permission issues are resolved internally, only returns the AR session starts successfully.
	 */
	UFUNCTION(BlueprintCallable, Category = "", meta = (Latent, LatentInfo = "LatentInfo", WorldContext = "WorldContextObject"))
	virtual void StartARSessionLatent(UObject* WorldContextObject, UARSessionConfig* SessionConfig, FLatentActionInfo LatentInfo) PURE_VIRTUAL(UARDependencyHandler::StartARSessionLatent, );
};

#undef UE_API
