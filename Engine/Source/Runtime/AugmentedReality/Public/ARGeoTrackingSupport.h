// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARBlueprintProxy.h"
#include "Misc/Optional.h"
#include "ARTypes.h"
#include "Features/IModularFeature.h"

#include "ARGeoTrackingSupport.generated.h"

#define UE_API AUGMENTEDREALITY_API


class FCheckGeoTrackingAvailabilityAsyncTask;
class FGetGeoLocationAsyncTask;


UENUM(BlueprintType)
enum class EARGeoTrackingState : uint8
{
	// The session is initializing geo tracking.
	Initializing,
	
	// Geo tracking is localized.
	Localized,
	
	// Geo tracking is attempting to localize against a map.
	Localizing,
	
	// Geo tracking is not available.
	NotAvailable,
};


UENUM(BlueprintType)
enum class EARGeoTrackingStateReason : uint8
{
	// No issues reported.
	None,
	
	// Geo tracking is not available at the location.
	NotAvailableAtLocation,
	
	// Geo tracking needs location permissions from the user.
	NeedLocationPermissions,
	
	// The user is pointing the device too low to use geo tracking.
	DevicePointedTooLow,
	
	// The session is unsure of the device’s pose in the physical environment.
	WorldTrackingUnstable,
	
	// The framework is waiting for a position for the user.
	WaitingForLocation,
	
	// The framework is actively attempting to download localization imagery.
	GeoDataNotLoaded,
	
	// The framework failed to match its localization imagery with the device’s camera captures.
	VisualLocalizationFailed,
	
	// The framework is waiting for the availability check.
	WaitingForAvailabilityCheck,
};


UENUM(BlueprintType)
enum class EARGeoTrackingAccuracy : uint8
{
	// Geo-tracking accuracy is undetermined.
	Undetermined,
	
	// Geo-tracking accuracy is low.
	Low,
	
	// Geo-tracking accuracy is average.
	Medium,
	
	// Geo-tracking accuracy is high.
	High,
};


/**
 * Interface class for Geo tracking related features.
 */
UCLASS(MinimalAPI, BlueprintType, Abstract, Category="AR|Geo Tracking")
class UARGeoTrackingSupport : public UObject, public IModularFeature
{
	GENERATED_BODY()
	
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("ARGeoTrackingSupport");
		return FeatureName;
	}
	
	/** @return the interface object to support Geo tracking, return null on platforms don't support the feature. */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	static UE_API class UARGeoTrackingSupport* GetGeoTrackingSupport();
	
	/**
	 * @return the async task to check Geo tracking availability at the user's current location.
	 * Can return null if the feature is not supported.
	 */
	UE_API virtual TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> CheckGeoTrackingAvailability(FString& OutError);
	
	/**
	 * @return the async task to check Geo tracking availability at a specific location.
	 * Can return null if the feature is not supported.
	 */
	UE_API virtual TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> CheckGeoTrackingAvailability(float Longitude, float Latitude, FString& OutError);
	
	/**
	 * @return the async task to convert a UE4 world position into a Geo location.
	 * Can return null if the feature is not supported.
	 */
	UE_API virtual TSharedPtr<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe> GetGeoLocationAtWorldPosition(const FVector& WorldPosition, FString& OutError);
	
	/**
	 * @return the current session's Geo tracking state.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	virtual EARGeoTrackingState GetGeoTrackingState() const { return EARGeoTrackingState::NotAvailable; }
	
	/**
	 * @return the current session's Geo tracking state reason.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	virtual EARGeoTrackingStateReason GetGeoTrackingStateReason() const { return EARGeoTrackingStateReason::NotAvailableAtLocation; }
	
	/**
	 * @return the current session's Geo tracking state accuracy.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	virtual EARGeoTrackingAccuracy GetGeoTrackingAccuracy() const { return EARGeoTrackingAccuracy::Undetermined; }
	
	/**
	 * @return add an Geo anchor at a specific location.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	virtual bool AddGeoAnchorAtLocation(float Longitude, float Latitude, FString OptionalAnchorName) { return false; }
	
	/**
	 * @return add an Geo anchor at a specific location with an altitude (in meters).
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking")
	virtual bool AddGeoAnchorAtLocationWithAltitude(float Longitude, float Latitude, float AltitudeMeters, FString OptionalAnchorName) { return false; }
};


/**
 * Async task to check Geo tracking availability.
 */
class FCheckGeoTrackingAvailabilityAsyncTask : public FARAsyncTask
{
public:
	bool IsAvailable() const { return bIsAvailable; }
	
	/**
	 * Finish the task with the availability result.
	 */
	UE_API void FinishWithAvailability(bool bInIsAvailable);
	
	/**
	 * Finish the task with an error.
	 */
	UE_API void FinishWithError(const FString& InError);
	
private:
	bool bIsAvailable = false;
};


/**
 * Blueprint async task to check Geo tracking availability.
 */
UCLASS(MinimalAPI)
class UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy : public UARBaseAsyncTaskBlueprintProxy
{
	GENERATED_BODY()
	
public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FGeoTrackingAvailabilityDelegate, bool, bIsAvailable, FString, Error);
	
	UPROPERTY(BlueprintAssignable)
	FGeoTrackingAvailabilityDelegate OnSuccess;
	
	UPROPERTY(BlueprintAssignable)
	FGeoTrackingAvailabilityDelegate OnFailed;

	/**
	 * Check Geo tracking availability at the user's current location.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking", meta=(BlueprintInternalUseOnly="true", WorldContext = "WorldContextObject"))
	static UE_API UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy* CheckGeoTrackingAvailability(UObject* WorldContextObject);
	
	/**
	 * Check Geo tracking availability at a specific Geo location.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking", meta=(BlueprintInternalUseOnly="true", WorldContext = "WorldContextObject"))
	static UE_API UCheckGeoTrackingAvailabilityAsyncTaskBlueprintProxy* CheckGeoTrackingAvailabilityAtLocation(UObject* WorldContextObject, float Longitude, float Latitude);
	
private:
	// UBlueprintAsyncActionBase interface
	UE_API void Activate() override;
	UE_API void ReportSuccess() override;
	UE_API void ReportFailure() override;
	//~UBlueprintAsyncActionBase interface
	
	TOptional<float> Longitude;
	TOptional<float> Latitude;
	
	/** The async task to check during Tick() */
	TSharedPtr<FCheckGeoTrackingAvailabilityAsyncTask, ESPMode::ThreadSafe> MyTask;
	
	FString Error;
};


/**
 * Async task to convert Geo location.
 */
class FGetGeoLocationAsyncTask : public FARAsyncTask
{
public:
	/**
	 * Finish the task with a Geo location.
	 */
	UE_API void FinishWithGeoLocation(float InLongitude, float InLatitude, float InAltitude);
	
	/**
	 * Finish the task with an error.
	 */
	UE_API void FinishWithError(const FString& InError);
	
	/**
	 * @return the longitude of the converted Geo location.
	 */
	float GetLongitude() const { return Longitude; }
	
	/**
	 * @return the latitude of the converted Geo location.
	 */
	float GetLatitude() const { return Latitude; }
	
	/**
	 * @return the altitude of the converted Geo location.
	 */
	float GetAltitude() const { return Altitude; }
	
private:
	float Longitude = 0.f;
	float Latitude = 0.f;
	float Altitude = 0.f;
};


/**
 * Blueprint async task to convert Geo location.
 */
UCLASS(MinimalAPI)
class UGetGeoLocationAsyncTaskBlueprintProxy : public UARBaseAsyncTaskBlueprintProxy
{
	GENERATED_BODY()
	
public:
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams(FGetGeoLocationDelegate, float, Longitude, float, Latitude, float, Altitude, FString, Error);
	
	UPROPERTY(BlueprintAssignable)
	FGetGeoLocationDelegate OnSuccess;
	
	UPROPERTY(BlueprintAssignable)
	FGetGeoLocationDelegate OnFailed;

	/**
	 * Convert a position in UE4 world space into a Geo location.
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Geo Tracking", meta=(BlueprintInternalUseOnly="true", WorldContext = "WorldContextObject"))
	static UE_API UGetGeoLocationAsyncTaskBlueprintProxy* GetGeoLocationAtWorldPosition(UObject* WorldContextObject, const FVector& WorldPosition);
	
private:
	// UBlueprintAsyncActionBase interface
	UE_API void Activate() override;
	UE_API void ReportSuccess() override;
	UE_API void ReportFailure() override;
	//~UBlueprintAsyncActionBase interface
	
	FVector WorldPosition = FVector::ZeroVector;
	
	/** The async task to check during Tick() */
	TSharedPtr<FGetGeoLocationAsyncTask, ESPMode::ThreadSafe> MyTask;
	
	FString Error;
};

#undef UE_API
