// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ARTypes.h"
#include "UObject/ObjectMacros.h"
#include "ARComponent.h"
#include "ARActor.h"
#include "ARLifeCycleComponent.generated.h"

#define UE_API AUGMENTEDREALITY_API

class USceneComponent;

UCLASS(MinimalAPI, BlueprintType, Experimental, meta = (BlueprintSpawnableComponent), ClassGroup = "AR Gameplay")
class UARLifeCycleComponent : public USceneComponent
{
	GENERATED_BODY()

	UE_API virtual void OnComponentCreated() override;
	UE_API virtual void DestroyComponent(bool bPromoteChildren = false) override;

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FRequestSpawnARActorDelegate, UClass*, FGuid);
	static UE_API FRequestSpawnARActorDelegate RequestSpawnARActorDelegate;
	
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnSpawnARActorDelegate, AARActor*, UARComponent*, FGuid);
	static UE_API FOnSpawnARActorDelegate OnSpawnARActorDelegate;
	
	DECLARE_MULTICAST_DELEGATE_OneParam(FRequestDestroyARActorDelegate, AARActor*);
	static UE_API FRequestDestroyARActorDelegate RequestDestroyARActorDelegate;
	
	/** Called when an AR actor is spawned on the server */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FInstanceARActorSpawnedDelegate, UClass*, ComponentClass, FGuid, NativeID, AARActor*, SpawnedActor);
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "OnARActorSpawned"))
	FInstanceARActorSpawnedDelegate OnARActorSpawnedDelegate;
	
	/** Called just before the AR actor is destroyed on the server */
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FInstanceARActorToBeDestroyedDelegate, AARActor*, Actor);
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "OnARActorToBeDestroyed"))
	FInstanceARActorToBeDestroyedDelegate OnARActorToBeDestroyedDelegate;
	
protected:
	UE_API virtual void OnUnregister() override;
	
private:
	UE_API void CallInstanceRequestSpawnARActorDelegate(UClass* Class, FGuid NativeID);
	UE_API void CallInstanceRequestDestroyARActorDelegate(AARActor* Actor);
	
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerSpawnARActor(UClass* ComponentClass, FGuid NativeID);
	
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerDestroyARActor(AARActor* Actor);
	
	FDelegateHandle SpawnDelegateHandle;
	FDelegateHandle DestroyDelegateHandle;
};

#undef UE_API
