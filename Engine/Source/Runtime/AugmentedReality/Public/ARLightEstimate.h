// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "Math/MathFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ARLightEstimate.generated.h"

#define UE_API AUGMENTEDREALITY_API

struct FFrame;


UCLASS(BlueprintType, Experimental, Category="AR AugmentedReality|Light Estimation")
class UARLightEstimate : public UObject
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI, BlueprintType, Category = "AR AugmentedReality|Light Estimation")
class UARBasicLightEstimate : public UARLightEstimate
{
	GENERATED_BODY()
	
public:
	UE_API void SetLightEstimate(float InAmbientIntensityLumens, float InColorTemperatureKelvin);
	
	UE_API void SetLightEstimate(FVector InRGBScaleFactor, float InPixelIntensity);

	UE_API void SetLightEstimate(float InColorTemperatureKelvin, FLinearColor InAmbientColor);

	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Light Estimation")
	UE_API float GetAmbientIntensityLumens() const;
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Light Estimation")
	UE_API float GetAmbientColorTemperatureKelvin() const;
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Light Estimation")
	UE_API FLinearColor GetAmbientColor() const;
	
private:
	UPROPERTY()
	float AmbientIntensityLumens;
	
	UPROPERTY()
	float AmbientColorTemperatureKelvin;
	
	UPROPERTY()
	FLinearColor AmbientColor;
};

#undef UE_API
