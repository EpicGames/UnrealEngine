// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "AROriginActor.generated.h"

#define UE_API AUGMENTEDREALITY_API

/**
 * Simple actor that is spawned at the origin for AR systems that want to hang components on an actor
 * Spawned as a custom class for easier TObjectIterator use
 */
UCLASS(MinimalAPI, BlueprintType)
class AAROriginActor :
	public AActor
{
	GENERATED_UCLASS_BODY()
public:
	/** Used by the AR system to get the origin actor for the current world */
	static UE_API AAROriginActor* GetOriginActor();
};

#undef UE_API
