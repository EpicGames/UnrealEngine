// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARPin.generated.h"

#define UE_API AUGMENTEDREALITY_API

class FARSupportInterface ;
class USceneComponent;

UCLASS(MinimalAPI, BlueprintType, Experimental, Category="AR AugmentedReality")
class UARPin : public UObject
{
	GENERATED_BODY()
	
public:
	UE_API virtual void InitARPin( const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystemOwner, USceneComponent* InComponentToPin, const FTransform& InLocalToTrackingTransform, UARTrackedGeometry* InTrackedGeometry, const FName InDebugName );

	/**
	 * Maps from a Pin's Local Space to the Tracking Space.
	 * Mapping the origin from the Pin's Local Space to Tracking Space
	 * yield the Pin's position in Tracking Space.
	 */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Pin")
	UE_API FTransform GetLocalToTrackingTransform() const;
	
	
	/**
	 * Convenience function. Same as LocalToTrackingTransform, but
	 * appends the TrackingToWorld Transform.
	 */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Pin")
	UE_API FTransform GetLocalToWorldTransform() const;

	/**
	 * Return the current tracking state of this Pin.
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Pin")
	UE_API EARTrackingState GetTrackingState() const;
	
	/**
	 * The TrackedGeometry (if any) that this this pin is being "stuck" into.
	 */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Pin")
	UE_API UARTrackedGeometry* GetTrackedGeometry() const;
	
	/** @return the PinnedComponent that this UARPin is pinning to the TrackedGeometry */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Pin")
	UE_API USceneComponent* GetPinnedComponent() const;
	
	UFUNCTION()
	UE_API virtual void DebugDraw( UWorld* World, const FLinearColor& Color, float Scale = 5.0f, float PersistForSeconds = 0.0f) const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Pin")
	UE_API FName GetDebugName() const;
	
	UE_API void SetOnARTrackingStateChanged( const FOnARTrackingStateChanged& InHandler );
	
	UE_API void SetOnARTransformUpdated( const FOnARTransformUpdated& InHandler );

	void SetNativeResource(void* InNativeResource)
	{
		NativeResource = InNativeResource;
	}

	void* GetNativeResource()
	{
		return NativeResource;
	}
	
public:
	
	UE_API FTransform GetLocalToTrackingTransform_NoAlignment() const;
	
	/** Notify the ARPin about changes to how it is being tracked. */
	UE_API void OnTrackingStateChanged(EARTrackingState NewTrackingState);
	
	/** Notify this UARPin that the transform of the Pin has changed */
	UE_API void OnTransformUpdated(const FTransform& NewLocalToTrackingTransform);
	
	/** Notify the UARPin that the AlignmentTransform has changing. */
	UE_API void UpdateAlignmentTransform( const FTransform& NewAlignmentTransform );
	
	void SetPinnedComponent(USceneComponent* InComponentToPin)
	{
		PinnedComponent = InComponentToPin;
	}
	
protected:
	UE_API TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> GetARSystem() const;
	
private:
	static UE_API uint32 DebugPinId;
	
	UPROPERTY()
	TObjectPtr<UARTrackedGeometry> TrackedGeometry;
	
	UPROPERTY()
	TObjectPtr<USceneComponent> PinnedComponent;
	
	UPROPERTY()
	FTransform LocalToTrackingTransform;
	
	UPROPERTY()
	FTransform LocalToAlignedTrackingTransform;

	UPROPERTY()
	EARTrackingState TrackingState;
	
	TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> ARSystem;
	
	FName DebugName;
	
	UPROPERTY(BlueprintAssignable, Category="AR AugmentedReality|Pin")
	FOnARTrackingStateChanged OnARTrackingStateChanged;
	
	UPROPERTY(BlueprintAssignable, Category="AR AugmentedReality|Pin")
	FOnARTransformUpdated OnARTransformUpdated;
	
	// The native resource pointer on the AR platform.
	void* NativeResource;
};

#undef UE_API
