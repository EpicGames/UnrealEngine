// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ARTrackable.h"
#include "ARComponent.h"
#include "Engine/DataAsset.h"

#include "ARSessionConfig.generated.h"

#define UE_API AUGMENTEDREALITY_API

class UMaterialInterface;

/** Options for how the scene’s coordinate system is constructed. This feature is used by ARKit. */
UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARWorldAlignment : uint8
{
	/** The coordinate system is aligned with gravity, defined by the vector (0, -1, 0). Origin is the initial position of the device. */
	Gravity,

	/**
	 * The coordinate system is aligned with gravity, defined by the vector (0, -1, 0),
	 * and compass heading based on True North, defined by the vector (0, 0, -1). Origin is the initial position of the device.
	 */
	GravityAndHeading,

	/** The coordinate system matches the camera's orientation. This option is recommended for Face AR. */
	Camera
};

/** Options for the tracking type of the session. All AR platforms use this structure but only some session tracking are supported on each platform. The options are mutually exclusive.  */
UENUM(BlueprintType, Category = "AR AugmentedReality", meta = (Experimental))
enum class EARSessionType : uint8
{
	/** No tracking in the session. */
	None,

	/** A session where only the orientation of the device is tracked. ARKit supports this type of tracking.*/
	Orientation,

	/** A session where the position and orientation of the device is tracked relative to objects in the environment. All platforms support this type of tracking. */
	World,

	/** A session where only faces are tracked. ARKit and ARCore support this type of tracking using the front-facing camera.*/
	Face,

	/** A session where only images supplied by the app are tracked. There is no world tracking. ARKit supports this type of tracking. */
    Image,

	/** A session where objects are scanned for object detection in a later World Tracking session. ARKit supports this type of tracking. */
	ObjectScanning,
	
	/** A session where human poses in 3D are tracked. ARKit supports this type of tracking using the rear-facing camera. */
	PoseTracking,
	
	/** A session where geographic locations are tracked. ARKit supports this type of tracking. */
	GeoTracking,
};

/** Options for how flat surfaces are detected. This feature is used by ARCore and ARKit. */
UENUM(BlueprintType, Category = "AR AugmentedReality", meta = (Experimental, Bitflags))
enum class EARPlaneDetectionMode : uint8
{
	/** Disables plane detection. */
	None = 0,
	
	/* Detects horizontal, flat surfaces. */
	HorizontalPlaneDetection = 1,

	/* Detects vertical, flat surfaces. */
	VerticalPlaneDetection = 2
};
ENUM_CLASS_FLAGS(EARPlaneDetectionMode);

/** Options for how light is estimated based on the camera capture. This feature is used by ARCore and ARKit. */
UENUM(BlueprintType, Category = "AR AugmentedReality", meta = (Experimental))
enum class EARLightEstimationMode : uint8
{
	/** Disables light estimation. */
	None = 0,

	/** Estimates an ambient light. */
	AmbientLightEstimate = 1,

	/**
	* Estimates a directional light of the environment with an additional key light.
	* Currently not supported.
	*/
	DirectionalLightEstimate = 2
};

/** Options for how the Unreal frame synchronizes with camera image updates. This feature is used by ARCore. */
UENUM(BlueprintType, Category = "AR AugmentedReality", meta = (Experimental))
enum class EARFrameSyncMode : uint8
{
	/** Unreal tick will be synced with the camera image update rate. */
	SyncTickWithCameraImage = 0,

	/** Unreal tick will not related to the camera image update rate. */
	SyncTickWithoutCameraImage = 1,
};

/**
 * Options for how environment textures are generated. This feature is used by ARKit.
 */
UENUM(BlueprintType)
enum class EAREnvironmentCaptureProbeType : uint8
{
	/** Disables environment texture generation. */
	None,

	/** The app specifies where the environment capture probes are located and their size. */
	Manual,

	/** The AR system automatically creates and places the environment capture probes. */
	Automatic
};

/**
 * Options for the kind of data to update during Face Tracking. This feature is used by ARKit.
 */

UENUM(BlueprintType)
enum class EARFaceTrackingUpdate : uint8
{
	/** Both curves and geometry are updated. This is useful for mesh visualization. */
	CurvesAndGeo,

	/** Only the curve data is updated. */
	CurvesOnly
};

/**
 * Options for more tracking features to be enabled for the session, in addition to what is already defined in the project’s @EARSessionType.
 */

UENUM(BlueprintType)
enum class EARSessionTrackingFeature : uint8
{
	/** No additional features are enabled. */
	None,
	
	/** Adds tracking for 2D human poses to the session. This feature is used by ARKit. */
	PoseDetection2D,
	
	/** Uses person segmentation for occlusion in the session. This feature is used by ARKit. */
	PersonSegmentation,
	
	/** Uses person segmentation with depth information for occlusion in the session. This feature is used by ARKit. */
	PersonSegmentationWithDepth,
	
	/** Uses scene depth for occlusion while tracking in the session. This feature is used by ARCore and ARKit. */
	SceneDepth,
	
	/** Uses smoothed scene depth for occlusion while tracking in the session. This feature is used by ARKit. */
	SmoothedSceneDepth,
};

/**
 * Options for how the scene should be reconstructed. This feature is used by ARKit.
 */
UENUM(BlueprintType)
enum class EARSceneReconstruction : uint8
{
	/** Disables scene reconstruction*/
	None,
	
	/** Creates a mesh approximation of the environment. */
	MeshOnly,
	
	/** Creates a mesh approximation of the environment and classifies the objects constructed. */
	MeshWithClassification,
};

/**
 * An Unreal Data Asset that defines what features are used in the AR session.
 */
UCLASS(MinimalAPI, BlueprintType, Category="AR Settings")
class UARSessionConfig : public UDataAsset
{
	GENERATED_BODY()

public:
	/** The constructor for the AR Session Config Data Asset. */
	UE_API UARSessionConfig();
	
public:
	/** @see EARWorldAlignment */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARWorldAlignment GetWorldAlignment() const;

	/** @see SessionType */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARSessionType GetSessionType() const;

	/** @see PlaneDetectionMode */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARPlaneDetectionMode GetPlaneDetectionMode() const;

	/** @see LightEstimationMode */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARLightEstimationMode GetLightEstimationMode() const;

	/** @see FrameSyncMode */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARFrameSyncMode GetFrameSyncMode() const;

	/** @see bEnableAutomaticCameraOverlay */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API bool ShouldRenderCameraOverlay() const;

	/** @see bEnableAutomaticCameraTracking */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API bool ShouldEnableCameraTracking() const;

	/** @see bEnableAutoFocus */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API bool ShouldEnableAutoFocus() const;

	/** @see bEnableAutoFocus */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetEnableAutoFocus(bool bNewValue);

	/** @see bResetCameraTracking */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API bool ShouldResetCameraTracking() const;
	
	/** @see bResetCameraTracking */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetResetCameraTracking(bool bNewValue);

	/** @see bResetTrackedObjects */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API bool ShouldResetTrackedObjects() const;

	/** @see bResetTrackedObjects */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetResetTrackedObjects(bool bNewValue);
	
	/** @see CandidateImages */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API const TArray<UARCandidateImage*>& GetCandidateImageList() const;

	// Add a new CandidateImage to the ARSessionConfig.
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void AddCandidateImage(UARCandidateImage* NewCandidateImage);
    
	// Remove a candidate image from the ARSessionConfig, by pointer, note the image object must match, not the content of the image.
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void RemoveCandidateImage(UARCandidateImage* CandidateImage);

	// Remove a candidate image from the ARSessionConfig, by index.
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void RemoveCandidateImageAtIndex(int Index);

	// Remove all candidate images from the ARSessionConfig
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void ClearCandidateImages();

	/** @see MaxNumSimultaneousImagesTracked */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
    UE_API int32 GetMaxNumSimultaneousImagesTracked() const;
	
	/** @see EnvironmentCaptureProbeType */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EAREnvironmentCaptureProbeType GetEnvironmentCaptureProbeType() const;
	
	/** @see WorldMapData */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API const TArray<uint8>& GetWorldMapData() const;
	/** @see WorldMapData */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetWorldMapData(TArray<uint8> WorldMapData);

	/** @see CandidateObjects */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API const TArray<UARCandidateObject*>& GetCandidateObjectList() const;
	/** @see CandidateObjects */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetCandidateObjectList(const TArray<UARCandidateObject*>& InCandidateObjects);
	/** @see CandidateObjects */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void AddCandidateObject(UARCandidateObject* CandidateObject);

	/** @see DesiredVideoFormat */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API FARVideoFormat GetDesiredVideoFormat() const;
	/** @see DesiredVideoFormat */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetDesiredVideoFormat(FARVideoFormat NewFormat);

	/** @see FaceTrackingDirection */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARFaceTrackingDirection GetFaceTrackingDirection() const;
	/** @see FaceTrackingDirection */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetFaceTrackingDirection(EARFaceTrackingDirection InDirection);
	
	/** @see FaceTrackingUpdate */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARFaceTrackingUpdate GetFaceTrackingUpdate() const;
	/** @see FaceTrackingUpdate */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetFaceTrackingUpdate(EARFaceTrackingUpdate InUpdate);

	/** @see EnabledSessionTrackingFeatures */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARSessionTrackingFeature GetEnabledSessionTrackingFeature() const;
	
	/** @see SceneReconstructionMethod */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API EARSceneReconstruction GetSceneReconstructionMethod() const;
	
	/** @see EnabledSessionTrackingFeatures */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetSessionTrackingFeatureToEnable(EARSessionTrackingFeature InSessionTrackingFeature);
	
	/** @see SceneReconstructionMethod */
	UFUNCTION(BlueprintCallable, Category = "AR Settings")
	UE_API void SetSceneReconstructionMethod(EARSceneReconstruction InSceneReconstructionMethod);

	/** @see bHorizontalPlaneDetection */
	bool ShouldDoHorizontalPlaneDetection() const { return bHorizontalPlaneDetection; }
	/** @see bVerticalPlaneDetection */
	bool ShouldDoVerticalPlaneDetection() const { return bVerticalPlaneDetection; }
	
	/** @see  SerializedARCandidateImageDatabase */
	UE_API const TArray<uint8>& GetSerializedARCandidateImageDatabase() const;

	/** @see PlaneComponentClass */
	UE_API UClass* GetPlaneComponentClass(void) const;
	/** @see PointComponentClass */
	UE_API UClass* GetPointComponentClass(void) const;
	/** @see FaceComponentClass */
	UE_API UClass* GetFaceComponentClass(void) const;
	/** @see ImageComponentClass */
	UE_API UClass* GetImageComponentClass(void) const;
	/** @see QRCodeComponentClass */
	UE_API UClass* GetQRCodeComponentClass(void) const;
	/** @see PoseComponentClass */
	UE_API UClass* GetPoseComponentClass(void) const;
	/** @see EnvironmentProbeComponentClass */
	UE_API UClass* GetEnvironmentProbeComponentClass(void) const;
	/** @see ObjectComponentClass */
	UE_API UClass* GetObjectComponentClass(void) const;
	/** @see MeshComponentClass */
	UE_API UClass* GetMeshComponentClass(void) const;
	/** @see GeoAnchorComponentClass */
	UE_API UClass* GetGeoAnchorComponentClass(void) const;
	
	/** @see DefaultMeshMaterial */
	UMaterialInterface* GetDefaultMeshMaterial() const { return DefaultMeshMaterial; }
	/** @see DefaultWireframeMeshMaterial */
	UMaterialInterface* GetDefaultWireframeMeshMaterial() const { return DefaultWireframeMeshMaterial; }
	
	/** @see MaxNumberOfTrackedFaces */
	int32 GetMaxNumberOfTrackedFaces() const { return MaxNumberOfTrackedFaces; }
	
	/** Boolean to determine whether the AR system should generate mesh data that can be used for rendering, collision, NavMesh, and more. This feature is used by OpenXR, Windows Mixed Reality. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bGenerateMeshDataFromTrackedGeometry;

	/** Boolean to determine whether the AR system should generate collision data from the mesh data. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bGenerateCollisionForMeshData;

	/** Boolean to determine whether the AR system should generate collision data from the mesh data. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bGenerateNavMeshForMeshData;

	/** Boolean to determine whether the AR system should render the mesh data as occlusion meshes. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bUseMeshDataForOcclusion;

	/** Boolean to determine whether the AR system should render the mesh data as wireframe.  It is reccomended to simply set the DefaultMeshMaterial to whatever is desired, including a wireframe material and ignore this setting (there is no good reason for this to exist as a special case).*/
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bRenderMeshDataInWireframe;

	/** Boolean to determine whether the AR system should track scene objects: @see EARObjectClassification::SceneObject. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	bool bTrackSceneObjects;
	
	/** Boolean to determine whether to use the person segmentation results for occluding virtual content. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | Occlusion")
	bool bUsePersonSegmentationForOcclusion = true;
	
	/** Boolean to determine whether to use the scene depth information for occluding virtual content. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | Occlusion")
	bool bUseSceneDepthForOcclusion = false;
	
	/** Boolean to determine whether to automatically estimate and set the scale of a detected, or tracked, image. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | Image Tracking")
	bool bUseAutomaticImageScaleEstimation = true;
	
	/** Boolean to determine whether to use the standard onboarding UX, if the system supports it. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	bool bUseStandardOnboardingUX = false;
	
	/** @see bUseOptimalVideoFormat */
	UE_API bool ShouldUseOptimalVideoFormat() const;

private:
	//~ UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	//~ UObject interface

protected:
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	/** @see EARWorldAlignment */
	EARWorldAlignment WorldAlignment;

	/** @see EARSessionType */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	EARSessionType SessionType;

	/** @see EARPlaneDetectionMode */
	UPROPERTY()
	EARPlaneDetectionMode PlaneDetectionMode_DEPRECATED;
	
	/** Boolean to determine whether flat, horizontal surfaces are detected. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	bool bHorizontalPlaneDetection;
	
	/** Boolean to determine whether flat, vertical surfaces are detected. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	bool bVerticalPlaneDetection;

	/** Boolean to determine whether the camera should autofocus. Autofocus can cause subtle shifts in position for small objects at further camera distance. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	bool bEnableAutoFocus;

	/** @see EARLightEstimationMode */
	UPROPERTY(EditAnywhere, Category = "AR Settings")
	EARLightEstimationMode LightEstimationMode;

	/** @see EARFrameSyncMode */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "AR Settings")
	EARFrameSyncMode FrameSyncMode;

	/** Boolean to determine whether the AR camera feed should be drawn as an overlay. Defaults to true. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	bool bEnableAutomaticCameraOverlay;

	/** Boolean to determine whether the virtual camera should track the device movement. Defaults to true. This feature is used by ARCore and ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	bool bEnableAutomaticCameraTracking;

	/** Boolean to determine whether the AR system should reset camera tracking, such as its origin and transforms, when a new AR session starts. Defaults to true. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	bool bResetCameraTracking;
	
	/** Boolean to determine whether the AR system should remove any tracked objects when a new AR session starts. Defaults to true. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	bool bResetTrackedObjects;
	
	/** The list of candidate images to detect within the AR camera view. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings | Image Tracking")
	TArray<TObjectPtr<UARCandidateImage>> CandidateImages;

	/** The maximum number of images to track at the same time. Defaults to 1. This feature is used by ARKit. */
    UPROPERTY(EditAnywhere, Category="AR Settings | Image Tracking")
    int32 MaxNumSimultaneousImagesTracked;
	
	/** @see EAREnvironmentCaptureProbeType */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	EAREnvironmentCaptureProbeType EnvironmentCaptureProbeType;

	/** A previously saved world that will be loaded when the session starts. This feature is used by ARKit. */
	UPROPERTY(VisibleAnywhere, Category="AR Settings | World Mapping")
	TArray<uint8> WorldMapData;

	/** The list of candidate objects to search for in the scene. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	TArray<TObjectPtr<UARCandidateObject>> CandidateObjects;

	/**
	 * The desired video format (or the default, if not supported) that this session should use if the camera is enabled.
	 * Use GetSupportedVideoFormats to get a list of device-supported formats.
	 */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	FARVideoFormat DesiredVideoFormat;
	
	/** Boolean to determine whether to automatically pick the video format that best matches the device screen size */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	bool bUseOptimalVideoFormat = true;
	
	/** @see EARFaceTrackingDirection */
	UPROPERTY(EditAnywhere, Category="Face AR Settings")
	EARFaceTrackingDirection FaceTrackingDirection;

	/** @see EARFaceTrackingUpdate */
	UPROPERTY(EditAnywhere, Category="Face AR Settings")
	EARFaceTrackingUpdate FaceTrackingUpdate;
	
	/** The maximum number of faces to track simultaneously. This feature is used by ARKit. */
	UPROPERTY(EditAnywhere, Category="Face AR Settings")
	int32 MaxNumberOfTrackedFaces = 1;
	
	/** Data array for storing the cooked image database. This feature is used by ARCore. */
	UPROPERTY()
	TArray<uint8> SerializedARCandidateImageDatabase;
	
	/** @see EARSessionTrackingFeature */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	EARSessionTrackingFeature EnabledSessionTrackingFeature = EARSessionTrackingFeature::None;
	
	/** @see EARSceneReconstruction */
	UPROPERTY(EditAnywhere, Category="AR Settings")
	EARSceneReconstruction SceneReconstructionMethod = EARSceneReconstruction::None;

	/** @see UARPlaneComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARPlaneComponent> PlaneComponentClass;

	/** @see UARPointComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARPointComponent> PointComponentClass;

	/** @see UARFaceComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARFaceComponent> FaceComponentClass;

	/** @see UARImageComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARImageComponent> ImageComponentClass;

	/** @see UARQRCodeComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARQRCodeComponent> QRCodeComponentClass;

	/** @see UARPoseComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARPoseComponent> PoseComponentClass;

	/** @see UAREnvironmentProbeComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UAREnvironmentProbeComponent> EnvironmentProbeComponentClass;

	/** @see UARObjectComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARObjectComponent> ObjectComponentClass;

	/** @see UARMeshComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARMeshComponent> MeshComponentClass;
	
	/** @see UARGeoAnchorComponent */
	UPROPERTY(EditAnywhere, Category = "AR Gameplay")
	TSubclassOf<UARGeoAnchorComponent> GeoAnchorComponentClass;
	
	/** The default mesh material used by the generated mesh component. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	TObjectPtr<UMaterialInterface> DefaultMeshMaterial;

	/** The default mesh material used by the wireframe setting of the generated mesh component.  Note: It is recommended to ignore this wireframe feature and use a wireframe material for the DefaultMeshMaterial instead. */
	UPROPERTY(EditAnywhere, Category = "AR Settings | World Mapping")
	TObjectPtr<UMaterialInterface> DefaultWireframeMeshMaterial;
};

#undef UE_API
