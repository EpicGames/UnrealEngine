// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "ARSessionConfig.h"
#include "Serialization/Archive.h"
#include "Containers/UnrealString.h"

class IARSessionConfigCookSupport : public IModularFeature
{
public:
	static FName GetModularFeatureName()
	{
		static FName FeatureName = FName("ARSessionConfigCookingSupport");
		return FeatureName;
	}

	virtual void OnSerializeSessionConfig(UARSessionConfig* SessionConfig, FArchive& Ar, TArray<uint8>& SerializedARCandidateImageDatabase) = 0;
};
