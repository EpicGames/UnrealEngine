// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/GameMode.h"

#include "ARSharedWorldGameState.h"
#include "ARSharedWorldPlayerController.h"

#include "ARSharedWorldGameMode.generated.h"

#define UE_API AUGMENTEDREALITY_API

/** Per player information about what data has been sent to them */
USTRUCT(BlueprintType)
struct FARSharedWorldReplicationState
{
	GENERATED_BODY()
	
	FARSharedWorldReplicationState()
	{
		PreviewImageOffset = ARWorldOffset = 0;
	}
	
	/** The offset in the overall image data buffer */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 PreviewImageOffset;
	
	/** The offset in the overall ARWorld data buffer */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 ARWorldOffset;
};

UCLASS(MinimalAPI, BlueprintType)
class AARSharedWorldGameMode :
	public AGameMode
{
	GENERATED_UCLASS_BODY()

public:
	/** The size of the buffer to use per send request. Must be between 1 and 65535, though should not be max to avoid saturation */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="AR Shared World")
	int32 BufferSizePerChunk;
	
	/**
	 * Sets the image data for the shared world game session
	 *
	 * @param ImageData the blob to use as the image data
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, meta=(DisplayName="Set AR Preview Image Data"), Category="AR Shared World")
	UE_API void SetPreviewImageData(TArray<uint8> ImageData);
	
	/**
	 * Sets the image data for the shared world game session
	 *
	 * @param ARWorldData the blob to use as the AR world data
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, meta=(DisplayName="Set AR Shared World Data"), Category="AR Shared World")
	UE_API void SetARSharedWorldData(TArray<uint8> ARWorldData);
	
	/**
	 * Tells the game mode that the AR data is ready and should be replicated to all connected clients
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, meta=(DisplayName="Set AR World Sharing Is Ready"), Category="AR Shared World")
	UE_API void SetARWorldSharingIsReady();
	
	/**
	 * @return the game state for this game mode
	 */
	UFUNCTION(BlueprintCallable, BlueprintAuthorityOnly, Category="AR Shared World")
	UE_API AARSharedWorldGameState* GetARSharedWorldGameState();
	
private:
	UE_API virtual void Tick(float DeltaSeconds) override;
	UE_API virtual void Logout(AController* Exiting) override;

	/** Tracks whether the data should be sent to all clients or not */
	bool bShouldSendSharedWorldData;
	
	/** Holds the progress for each player that is being replicated to */
	TMap<AARSharedWorldPlayerController*, FARSharedWorldReplicationState> PlayerToReplicationStateMap;
	
	/** Scratch buffer for sending chunks, cached to avoid allocation churn */
	TArray<uint8> SendBuffer;
};

#undef UE_API
