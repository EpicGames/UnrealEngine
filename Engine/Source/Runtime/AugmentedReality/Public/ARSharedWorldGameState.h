// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/GameState.h"
#include "ARSharedWorldGameState.generated.h"

#define UE_API AUGMENTEDREALITY_API

UCLASS(MinimalAPI, BlueprintType)
class AARSharedWorldGameState :
	public AGameState
{
	GENERATED_UCLASS_BODY()

public:
	/** Each client and the host have a copy of the shared world data */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	/** The image taken at the time of world saving for use when aligning the AR world later in the session */
	TArray<uint8> PreviewImageData;

	/** Each client and the host have a copy of the shared world data */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	TArray<uint8> ARWorldData;

	/** The size of the image that will be replicated to each client */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 PreviewImageBytesTotal;
	
	/** The size of the AR world data that will be replicated to each client */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 ARWorldBytesTotal;
	
	/** The amount of the preview image data that has been replicated to this client so far */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 PreviewImageBytesDelivered;
	
	/** The amount of the AR world data that has been replicated to this client so far */
	UPROPERTY(BlueprintReadOnly, Category="AR Shared World")
	int32 ARWorldBytesDelivered;
	
	/**
	 * Used to setup the initial values and size the arrays (client)
	 *
	 * @param PreviewImageSize the total size in bytes of the image data that will be sent
	 * @param ARWorldDataSize the total size in bytes of the AR world data that will be sent
	 */
	UE_API void InitSharedWorld(int32 PreviewImageSize, int32 ARWorldDataSize);
	
	/**
	 * Copies the buffer into the image data (client)
	 *
	 * @param Offset where in the buffer to start the copying
	 * @param Buffer the chunk of data to copy
	 * @param BufferSize the amount of data to copy
	 */
	UE_API void UpdatePreviewImageData(int32 Offset, const uint8* Buffer, int32 Size);

	/**
	 * Copies the buffer into the AR world data (client)
	 *
	 * @param Offset where in the buffer to start the copying
	 * @param Buffer the chunk of data to copy
	 * @param BufferSize the amount of data to copy
	 */
	UE_API void UpdateARWorldData(int32 Offset, const uint8* Buffer, int32 Size);

	UFUNCTION(BlueprintImplementableEvent, Category="AR Shared World", meta = (DisplayName = "OnARWorldMapIsReady", ScriptName = "OnARWorldMapIsReady"))
	UE_API void K2_OnARWorldMapIsReady();

private:
	UE_API void TriggerCompletionIfDone();
	
	bool bFiredCompletionEvent;
};

#undef UE_API
