// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerController.h"
#include "ARSharedWorldPlayerController.generated.h"

#define UE_API AUGMENTEDREALITY_API

class AARSharedWorldGameState;

UCLASS(MinimalAPI)
class AARSharedWorldPlayerController :
	public APlayerController
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Tells the server it is ready for receiving any shared world data
	 */
	UFUNCTION(Reliable, Server, WithValidation)
	UE_API void ServerMarkReadyForReceiving();
	
	/**
	 * Used to setup the initial values and size the arrays (client)
	 *
	 * @param PreviewImageSize the total size in bytes of the image data that will be sent
	 * @param ARWorldDataSize the total size in bytes of the AR world data that will be sent
	 */
	UFUNCTION(Reliable, Client, WithValidation)
	UE_API void ClientInitSharedWorld(int32 PreviewImageSize, int32 ARWorldDataSize);

	/**
	 * Copies the buffer into the image data (client)
	 *
	 * @param Offset where in the buffer to start the copying
	 * @param Buffer the chunk of data to copy
	 * @param BufferSize the amount of data to copy
	 */
	UFUNCTION(Reliable, Client, WithValidation)
	UE_API void ClientUpdatePreviewImageData(int32 Offset, const TArray<uint8>& Buffer);
	
	/**
	 * Copies the buffer into the AR world data (client)
	 *
	 * @param Offset where in the buffer to start the copying
	 * @param Buffer the chunk of data to copy
	 * @param BufferSize the amount of data to copy
	 */
	UFUNCTION(Reliable, Client, WithValidation)
	UE_API void ClientUpdateARWorldData(int32 Offset, const TArray<uint8>& Buffer);
	
	/** @return whether this player can start receiving the ar world data */
	bool IsReadyToReceive() const { return bIsReadyToReceive; }

private:
	UE_API virtual void Tick(float DeltaSeconds) override;

	UE_API AARSharedWorldGameState* GetGameState();
	UE_API bool IsGameStateReady();

	bool bIsReadyToReceive;
};

#undef UE_API
