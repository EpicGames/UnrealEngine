// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkyLightComponent.h"
#include "Engine/SkyLight.h"

#include "ARSkyLight.generated.h"

#define UE_API AUGMENTEDREALITY_API

class UAREnvironmentCaptureProbe;

/** This sky light class forces a refresh of the cube map data when an AR environment probe changes */
UCLASS(MinimalAPI)
class AARSkyLight :
	public ASkyLight
{
	GENERATED_UCLASS_BODY()

public:
	/** Sets the environment capture probe that this sky light is driven by */
	UFUNCTION(BlueprintCallable, Category="AR AugmentedReality|SkyLight")
	UE_API void SetEnvironmentCaptureProbe(UAREnvironmentCaptureProbe* InCaptureProbe);

private:
	UE_API virtual void Tick(float DeltaTime) override;

	UPROPERTY()
	TObjectPtr<UAREnvironmentCaptureProbe> CaptureProbe;

	/** The timestamp from the environment probe when we last updated the cube map */
	float LastUpdateTimestamp;
};

#undef UE_API
