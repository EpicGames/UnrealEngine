// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARSessionConfig.h"
#include "ARTextures.h"
#include "Engine/Engine.h" // for FWorldContext

#define UE_API AUGMENTEDREALITY_API

class IARSystemSupport;
class IXRTrackingSystem;

DECLARE_MULTICAST_DELEGATE(FARSystemOnSessionStarted);
DECLARE_MULTICAST_DELEGATE_OneParam(FARSystemOnAlignmentTransformUpdated, const FTransform&);

#define DECLARE_AR_SI_DELEGATE_FUNCS(DelegateName) \
public: \
	UE_API FDelegateHandle Add##DelegateName##Delegate_Handle(const F##DelegateName##Delegate& Delegate); \
	UE_API void Clear##DelegateName##Delegate_Handle(FDelegateHandle& Handle); \
	UE_API void Clear##DelegateName##Delegates(FDelegateUserObject Object);

/**
 * Composition Components for tracking system features
*/

class FARSupportInterface : public TSharedFromThis<FARSupportInterface, ESPMode::ThreadSafe>, public FGCObject, public IModularFeature
{
public:
	UE_API FARSupportInterface (IARSystemSupport* InARImplementation, IXRTrackingSystem* InXRTrackingSystem);
	UE_API virtual ~FARSupportInterface ();


	static FName GetModularFeatureName()
	{
		static const FName ModularFeatureName = FName("ARSystem");
		return ModularFeatureName;
	}

	UE_API void InitializeARSystem();
	UE_API IXRTrackingSystem* GetXRTrackingSystem();

	UE_API bool StartARGameFrame(FWorldContext& WorldContext);

	UE_API const FTransform& GetAlignmentTransform() const;
	UE_API const UARSessionConfig& GetSessionConfig() const;
	UE_API UARSessionConfig& AccessSessionConfig();

	/** \see UARBlueprintLibrary::GetTrackingQuality() */
	UE_API EARTrackingQuality GetTrackingQuality() const;
	/** \see UARBlueprintLibrary::GetTrackingQualityReason() */
	UE_API EARTrackingQualityReason GetTrackingQualityReason() const;
	/** \see UARBlueprintLibrary::StartARSession() */
	UE_API void StartARSession(UARSessionConfig* InSessionConfig);
	/** \see UARBlueprintLibrary::PauseARSession() */
	UE_API void PauseARSession();
	/** \see UARBlueprintLibrary::StopARSession() */
	UE_API void StopARSession();
	/** \see UARBlueprintLibrary::GetARSessionStatus() */
	UE_API FARSessionStatus GetARSessionStatus() const;
	/** \see UARBlueprintLibrary::IsSessionTypeSupported() */
	UE_API bool IsSessionTypeSupported(EARSessionType SessionType) const;

	/** \see UARBlueprintLibrary::ToggleARCapture() */
	UE_API bool ToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType);

	/** \see UARBlueprintLibrary::ToggleARCapture() */
	UE_API void SetEnabledXRCamera(bool bOnOff);
	/** \see UARBlueprintLibrary::ToggleARCapture() */
	UE_API FIntPoint ResizeXRCamera(const FIntPoint& InSize);

	/**
	 * \see UARBlueprintLibrary::SetAlignmentTransform()
	 * \see IARSystemSupport
	 * To understand the various spaces involved in Augmented Reality system, \see IARSystemSupport.
	 */
	UE_API void SetAlignmentTransform(const FTransform& InAlignmentTransform);

	/** \see UARBlueprintLibrary::LineTraceTrackedObjects() */
	UE_API TArray<FARTraceResult> LineTraceTrackedObjects(const FVector2D ScreenCoords, EARLineTraceChannels TraceChannels);
	/** \see UARBlueprintLibrary::LineTraceTrackedObjects() */
	UE_API TArray<FARTraceResult> LineTraceTrackedObjects(const FVector Start, const FVector End, EARLineTraceChannels TraceChannels);
	/** \see UARBlueprintLibrary::GetAllTrackedGeometries() */
	UE_API TArray<UARTrackedGeometry*> GetAllTrackedGeometries() const;
	/** \see UARBlueprintLibrary::GetAllPins() */
	UE_API TArray<UARPin*> GetAllPins() const;
	/**\see UARBlueprintLibrary::IsEnvironmentCaptureSupported() */
	UE_API bool IsEnvironmentCaptureSupported() const;
	/**\see UARBlueprintLibrary::AddEnvironmentCaptureProbe() */
	UE_API bool AddManualEnvironmentCaptureProbe(FVector Location, FVector Extent);
	/** Creates an async task that will perform the work in the background */
	UE_API TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> GetCandidateObject(FVector Location, FVector Extent) const;
	/** Creates an async task that will perform the work in the background */
	UE_API TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> SaveWorld() const;
	/** @return the current mapping status */
	UE_API EARWorldMappingState GetWorldMappingStatus() const;

	/** \see UARBlueprintLibrary::GetCurrentLightEstimate() */
	UE_API UARLightEstimate* GetCurrentLightEstimate() const;

	/** \see UARBlueprintLibrary::PinComponent() */
	UE_API UARPin* PinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None);
	/** \see UARBlueprintLibrary::PinComponentToTraceResult() */
	UE_API UARPin* PinComponent(USceneComponent* ComponentToPin, const FARTraceResult& HitResult, const FName DebugName = NAME_None);
	/** \see UARBlueprintLibrary::RemovePin() */
	UE_API void RemovePin(UARPin* PinToRemove);
	UE_API bool TryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InAnchorName, UARPin*& OutAnchor);

	/** \see UARBlueprintLibrary::GetSupportedVideoFormats() */
	UE_API TArray<FARVideoFormat> GetSupportedVideoFormats(EARSessionType SessionType = EARSessionType::World) const;

	/** @return the current point cloud data for the ar scene */
	UE_API TArray<FVector> GetPointCloud() const;

	/** \see UARBlueprintLibrary::AddRuntimeCandidateImage() */
	UE_API UARCandidateImage* AddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth);

	UE_API void* GetARSessionRawPointer();
	UE_API void* GetGameThreadARFrameRawPointer();
	
	/** \see UARBlueprintLibrary::IsSessionTrackingFeatureSupported() */
	UE_API bool IsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const;
	
	/** \see UARBlueprintLibrary::GetTracked2DPose() */
	UE_API TArray<FARPose2D> GetTracked2DPose() const;
	
	/** \see UARBlueprintLibrary::IsSceneReconstructionSupported() */
	UE_API bool IsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const;
	
	/** \see UARBlueprintLibrary::AddTrackedPointWithName() */
	UE_API bool AddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName);
	
	/** \see UARBlueprintLibrary::GetNumberOfTrackedFacesSupported() */
	UE_API int32 GetNumberOfTrackedFacesSupported() const;
	
	/** \see UARBlueprintLibrary::GetARTexture() */
	UE_API UARTexture* GetARTexture(EARTextureType TextureType) const;
	
	/** \see UARBlueprintLibrary::GetCameraIntrinsics() */
	UE_API bool GetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const;
	
	/** \see UARBlueprintLibrary::IsARSupported() */
	UE_API bool IsARAvailable() const;

	//~ FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FARSupportInterface");
	}
	//~ FGCObject

	// Pass through helpers to create the methods needed to add/remove delegates from the AR system
	DECLARE_AR_SI_DELEGATE_FUNCS(OnTrackableAdded)
	DECLARE_AR_SI_DELEGATE_FUNCS(OnTrackableUpdated)
	DECLARE_AR_SI_DELEGATE_FUNCS(OnTrackableRemoved)
	// End helpers

	/** Pin Interface */
	UE_API bool PinComponent(USceneComponent* ComponentToPin, UARPin* Pin);
	UE_API bool IsLocalPinSaveSupported() const;
	UE_API bool ArePinsReadyToLoad();
	UE_API void LoadARPins(TMap<FName, UARPin*>& LoadedPins);
	UE_API bool SaveARPin(FName InName, UARPin* InPin);
	UE_API void RemoveSavedARPin(FName InName);
	UE_API void RemoveAllSavedARPins();


	FARSystemOnSessionStarted OnARSessionStarted;
	FARSystemOnAlignmentTransformUpdated OnAlignmentTransformUpdated;

private:
	IARSystemSupport* ARImplemention;
	IXRTrackingSystem* XRTrackingSystem;

	/** Alignment transform between AR System's tracking space and Unreal's World Space. Useful in static lighting/geometry scenarios. */
	FTransform AlignmentTransform;
	TObjectPtr<UARSessionConfig> ARSettings;
};

#undef UE_API
