// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ARTypes.h"
#include "ARSessionConfig.h"
#include "Engine/Engine.h" // for FWorldContext

class UARPin;
class UARTexture;
class UARTrackedGeometry;
class UARSessionConfig;
class UARTextureCameraImage;
class UARTextureCameraDepth;
struct FARTraceResult;
enum class EARTextureType : uint8;

#define DEFINE_AR_DELEGATE_BASE(DelegateName) \
private: \
	F##DelegateName DelegateName##Delegates; \
public: \
	virtual FDelegateHandle Add##DelegateName##Delegate_Handle(const F##DelegateName##Delegate& Delegate) \
	{ \
		DelegateName##Delegates.Add(Delegate); \
		return Delegate.GetHandle(); \
	} \
	virtual void Clear##DelegateName##Delegate_Handle(FDelegateHandle& Handle) \
	{ \
		DelegateName##Delegates.Remove(Handle); \
		Handle.Reset(); \
	} \
	virtual void Clear##DelegateName##Delegates(FDelegateUserObject Object) \
	{ \
		DelegateName##Delegates.RemoveAll(Object); \
	}

//need to move refactor some class hierarchies to make this protected again
#define DEFINE_AR_DELEGATE_ONE_PARAM(DelegateName, Param1Type) \
	DEFINE_AR_DELEGATE_BASE(DelegateName) \
public: \
	virtual void Trigger##DelegateName##Delegates(Param1Type Param1) \
	{ \
		DelegateName##Delegates.Broadcast(Param1); \
	} \
public:


/**
 * Implement IARSystemSupport for any platform that wants to be an Unreal Augmented Reality System. e.g. AppleARKit, GoogleARCore.
 * This interface is included as part of the abstract class \c FARSupportInterface . The functions you must override
 * are coalesced here for clarity.
 * 
 *  Augmented Reality Spaces
 * -----------------------------------------------------------------------------------------------
 * Engineers working on supporting Augmented Reality must be aware of three spaces:
 *
 * TrackingSpace :
 *     This is the space defined by the underlying AR system (e.g. ARKit, ARCore, etc.)
 *     Unreal has no control over the origin of this space.
 *                     
 * AlignedTrackingSpace :
 *     To regain control of TrackingSpace, Unreal applies the AlignmentTransform so
 *     bend TrackingSpace to its will. If you are implementing IARSystemSupport, you
 *     will need to understand this transform and apply it accordingly.
 *
 * WorldSpace :
 *     This is Unreal's coordinate system. Coordinates from Tracking Space can be translated
 *     into WorldSpace by using the AlignmentTransform and the TrackingToWorldTransform.
 *
 *
 * \verbatim
 * [TrackingSpace]--(AlignmentTransform)-->[AlignedTrackingSpace]--(TrackingToWorld)-->[WorldSpace]
 * \endverbatim
 *
 */
class IARSystemSupport
{
public:
	/** Invoked after the base AR system has been initialized. */
	virtual void OnARSystemInitialized(){};

	virtual bool OnStartARGameFrame(FWorldContext& WorldContext) { return false;  };

	/** @return the tracking quality; if unable to determine tracking quality, return EARTrackingQuality::NotAvailable */
	virtual EARTrackingQuality OnGetTrackingQuality() const = 0;
	
	/** @return the reason of limited tracking quality; if the state is not limited, return EARTrackingQualityReason::None */
	virtual EARTrackingQualityReason OnGetTrackingQualityReason() const = 0;
	
	/**
	 * Start the AR system.
	 *
	 * @param SessionType The type of AR session to create
	 *
	 * @return true if the system was successfully started
	 */
	virtual void OnStartARSession(UARSessionConfig* SessionConfig) = 0;

	/** Stop the AR system but leave its internal state intact. */
	virtual void OnPauseARSession() = 0;

	/** Stop the AR system and reset its internal state; this task must succeed. */
	virtual void OnStopARSession() = 0;

	/** @return the info about whether the session is running normally or encountered some kind of error. */
	virtual FARSessionStatus OnGetARSessionStatus() const = 0;
	
	/** Returns true/false based on whether AR features are available */
	virtual bool IsARAvailable() const = 0;
	
	/**
	 * Set a transform that will align the Tracking Space origin to the World Space origin.
	 * This is useful for supporting static geometry and static lighting in AR.
	 * Note: Usually, an app will ask the user to select an appropriate location for some
	 * experience. This allows us to choose an appropriate alignment transform.
	 */
	virtual void OnSetAlignmentTransform(const FTransform& InAlignmentTransform) = 0;
	
	/**
	 * Trace all the tracked geometries and determine which have been hit by a ray cast from `ScreenCoord`.
	 * Only geometries specified in `TraceChannels` are considered.
	 *
	 * @return a list of all the geometries that were hit, sorted by distance
	 */
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects( const FVector2D ScreenCoord, EARLineTraceChannels TraceChannels ) = 0;
	virtual TArray<FARTraceResult> OnLineTraceTrackedObjects( const FVector Start, const FVector End, EARLineTraceChannels TraceChannels ) = 0;
	
	/** @return a TArray of all the tracked geometries known to your ar system */
	virtual TArray<UARTrackedGeometry*> OnGetAllTrackedGeometries() const = 0;
	
	/** @return a TArray of all the pins that attach components to TrackedGeometries */
	virtual TArray<UARPin*> OnGetAllPins() const = 0;

	/** @return whether the specified tracking type is supported by this device */
	virtual bool OnIsTrackingTypeSupported(EARSessionType SessionType) const = 0;

	virtual bool OnToggleARCapture(const bool bOnOff, const EARCaptureType CaptureType) { return false; };

	virtual void OnSetEnabledXRCamera(bool bOnOff) {};
	virtual FIntPoint OnResizeXRCamera(const FIntPoint& InSize) { return FIntPoint(0, 0); }

	/** @return the best available light estimate; nullptr if light estimation is inactive or not available */
	virtual UARLightEstimate* OnGetCurrentLightEstimate() const = 0;

	/**
	 * Given a scene component find the ARPin which it is pinned by, if any.
	 */
	virtual UARPin* FindARPinByComponent(const USceneComponent* Component) const = 0;

	/**
	 * Pin an Unreal Component to a location in the world.
	 * Optionally, associate with a TrackedGeometry to receive transform updates that effectively attach the component to the geometry.
	 *
	 * @return the UARPin object that is pinning the component to the world and (optionally) a TrackedGeometry
	 */
	virtual UARPin* OnPinComponent(USceneComponent* ComponentToPin, const FTransform& PinToWorldTransform, UARTrackedGeometry* TrackedGeometry = nullptr, const FName DebugName = NAME_None) = 0;
	
	/**
	 * Pin a component to an existing ARPin.  
	 * If the component is alreayd pinned it will first be removed from that pin, which will be left empty.
	 * If the target Pin is already in use the component previously pinned to it will be unpinned.
	 *
	 * @return true if the operation was successful
	 */
	virtual bool OnPinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin) = 0;

	/**
	 * Given a pin, remove it and stop updating the associated component based on the tracked geometry.
	 * The component in question will continue to track with the world, but will not get updates specific to a TrackedGeometry.
	 */
	virtual void OnRemovePin(UARPin* PinToRemove) = 0;

	/**
	* Given the platform specific native resource for a pin, create a pin.
	*
	* @return true when a pin is successfully created, otherwise false
	*/
	virtual bool OnTryGetOrCreatePinForNativeResource(void* InNativeResource, const FString& InPinName, UARPin*& OutPin) { OutPin = nullptr;  return false; }

	/** Tells the ARSystem to generate a capture probe at the specified location if supported */
	virtual bool OnAddManualEnvironmentCaptureProbe(FVector Location, FVector Extent) = 0;
	
	/** Generates a UARCandidateObject from the point cloud data within the location and its extent using an async task */
	virtual TSharedPtr<FARGetCandidateObjectAsyncTask, ESPMode::ThreadSafe> OnGetCandidateObject(FVector Location, FVector Extent) const = 0;

	/** Saves the AR world to a byte array using an async task */
	virtual TSharedPtr<FARSaveWorldAsyncTask, ESPMode::ThreadSafe> OnSaveWorld() const = 0;
	
	/** @return the current mapping status */
	virtual EARWorldMappingState OnGetWorldMappingStatus() const = 0;
	
	/** @return The list of supported video formats for this device and session type */
	virtual TArray<FARVideoFormat> OnGetSupportedVideoFormats(EARSessionType SessionType) const = 0;
	
	/** @return the current point cloud data for the ar scene */
	virtual TArray<FVector> OnGetPointCloud() const = 0;
	
	/** Add candidate image at runtime @return True if it added the iamge successfully */
	virtual bool OnAddRuntimeCandidateImage(UARSessionConfig* SessionConfig, UTexture2D* CandidateTexture, FString FriendlyName, float PhysicalWidth) = 0;

	virtual void* GetARSessionRawPointer() = 0;
	virtual void* GetGameThreadARFrameRawPointer() = 0;
	
	/** @return if a particular session feature is supported on this device */
	virtual bool OnIsSessionTrackingFeatureSupported(EARSessionType SessionType, EARSessionTrackingFeature SessionTrackingFeature) const { return false; }
	
	/** @return all the tracked 2D poses in AR */
	virtual TArray<FARPose2D> OnGetTracked2DPose() const { return {}; }
	
	/** @return the required scene construction method is supported */
	virtual bool OnIsSceneReconstructionSupported(EARSessionType SessionType, EARSceneReconstruction SceneReconstructionMethod) const { return false; }
	
	virtual bool OnAddTrackedPointWithName(const FTransform& WorldTransform, const FString& PointName, bool bDeletePointsWithSameName) { return false; }
	
	/** @return the max number of faces can be tracked at the same time */
	virtual int32 OnGetNumberOfTrackedFacesSupported() const { return 1; }
	
	/** @return the AR texture for the specified type */
	virtual UARTexture* OnGetARTexture(EARTextureType TextureType) const { return nullptr; }
	
	virtual ~IARSystemSupport()= default;

	/**
	 * Delegate called when an ar item is added to the scene
	 *
	 * @param Added the item that was added
	 */
	DEFINE_AR_DELEGATE_ONE_PARAM(OnTrackableAdded, UARTrackedGeometry* /* Added */);

	/**
	 * Delegate called when an ar item is updated
	 *
	 * @param Updated the item that was updated
	 */
	DEFINE_AR_DELEGATE_ONE_PARAM(OnTrackableUpdated, UARTrackedGeometry* /* Updated */);

	/**
	 * Delegate called when an ar item is removed from the scene
	 *
	 * @param Removed the item that was removed
	 */
	DEFINE_AR_DELEGATE_ONE_PARAM(OnTrackableRemoved, UARTrackedGeometry* /* Removed */);

	// ARPin Local Store support.
	// Some Platforms/Devices have the ability to persist AR Anchors (real world positiosn) to the device or user account.
	// They are saved and loaded with a string identifier.

	virtual bool IsLocalPinSaveSupported() const
	{
		return false;
	}

	virtual bool ArePinsReadyToLoad()
	{
		//UE_LOGF(LogHMD, Log, "Pin Local Store is not supported on this platform");
		return false;
	}

	virtual void LoadARPins(TMap<FName, UARPin*>& LoadedPins)
	{
		//UE_LOGF(LogHMD, Log, "Pin Local Store is not supported on this platform");
	}
	virtual bool SaveARPin(FName InName, UARPin* InPin)
	{
		//UE_LOGF(LogHMD, Log, "Pin Local Store is not supported on this platform");
		return false;
	}
	virtual void RemoveSavedARPin(FName InName)
	{
		//UE_LOGF(LogHMD, Log, "Pin Local Store is not supported on this platform");
	}
	virtual void RemoveAllSavedARPins()
	{
		//UE_LOGF(LogHMD, Log, "Pin Local Store is not supported on this platform");
	}
	
	virtual bool OnGetCameraIntrinsics(FARCameraIntrinsics& OutCameraIntrinsics) const { return false; }
};
