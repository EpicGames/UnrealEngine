// Copyright Epic Games, Inc. All Rights Reserved.

// Common implementation that can be shared by mnay ARSystemSupport implementations

#pragma once

#include "ARSystem.h"

#define UE_API AUGMENTEDREALITY_API

class FARSystemSupportBase : public IARSystemSupport
{
public:
	UE_API virtual bool OnPinComponentToARPin(USceneComponent* ComponentToPin, UARPin* Pin) override;
};




#undef UE_API
