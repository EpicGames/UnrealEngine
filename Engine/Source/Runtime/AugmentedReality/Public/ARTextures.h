// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/Texture.h"
#include "Engine/TextureCube.h"
#include "ARTextures.generated.h"

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARTextureType : uint8
{
	Unknown,
	CameraImage,
	CameraDepth,
	EnvironmentCapture,
	PersonSegmentationImage,
	PersonSegmentationDepth,
	SceneDepthMap,
	SceneDepthConfidenceMap,
};

/**
 * Base class for all AR texture types.
 * Derived from UTexture instead of UTexture2D because UTexture2D is all about streaming and source art
 * ? probably should have been UTexture2DDynamic
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UARTexture : public UTexture
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Factory function for creating a new AR  texture of a particular type
	 */
	template<class T>
	static T* CreateARTexture(EARTextureType InTextureType)
	{
		auto NewTexture = NewObject<T>();
		NewTexture->TextureType = InTextureType;
		check(NewTexture->TextureType != EARTextureType::Unknown);
		return NewTexture;
	}
	
	// UTexture interface implementation
	virtual float GetSurfaceWidth() const override { return Size.X; }
	virtual float GetSurfaceHeight() const override { return Size.Y; }
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual FGuid GetExternalTextureGuid() const override { return ExternalTextureGuid; }
	virtual ETextureClass GetTextureClass() const { return ETextureClass::Other2DNoSource; }
	// End UTexture interface
	
	/** The type of texture this is */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	EARTextureType TextureType = EARTextureType::Unknown;

	/** The timestamp this texture was captured at */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	float Timestamp;

	/** The guid of texture that gets registered as an external texture */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	FGuid ExternalTextureGuid;

	/** The width and height of the texture */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	FVector2f Size;
};

/**
 * Base class for all AR textures that represent the camera image data
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UARTextureCameraImage : public UARTexture
{
	GENERATED_UCLASS_BODY()
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARDepthQuality : uint8
{
	Unkown,
	/** Not suitable to use as part of a rendering pass or for scene reconstruction */
	Low,
	/** Suitable for rendering against or for use in scene reconstruction */
	High
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARDepthAccuracy : uint8
{
	Unkown,
	/** Suitable for gross sorting of depths */
	Approximate,
	/** Accurate depth values that match the physical world */
	Accurate
};

/**
 * Base class for all AR textures that represent the camera depth data
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UARTextureCameraDepth : public UARTexture
{
	GENERATED_UCLASS_BODY()

public:
	/** The quality of the depth information captured this frame */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	EARDepthQuality DepthQuality;

	/** The accuracy of the depth information captured this frame */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	EARDepthAccuracy DepthAccuracy;

	/** Whether or not the depth information is temporally smoothed */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	bool bIsTemporallySmoothed;
};

/**
 * Base class for all AR textures that represent the environment for lighting and reflection
 */
UCLASS(MinimalAPI, Abstract, BlueprintType)
class UAREnvironmentCaptureProbeTexture : public UTextureCube
{
	GENERATED_UCLASS_BODY()
	
public:
	// UTexture interface implementation
	virtual float GetSurfaceWidth() const override { return Size.X; }
	virtual float GetSurfaceHeight() const override { return Size.Y; }
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 6; }
	virtual FGuid GetExternalTextureGuid() const override { return ExternalTextureGuid; }
	// End UTexture interface
	
	/** The type of texture this is */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	EARTextureType TextureType;
	
	/** The timestamp this texture was captured at */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	float Timestamp;
	
	/** The guid of texture that gets registered as an external texture */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	FGuid ExternalTextureGuid;
	
	/** The width and height of the texture */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality", meta=(Experimental))
	FVector2f Size;
};
