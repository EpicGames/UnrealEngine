// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARTraceResult.generated.h"

#define UE_API AUGMENTEDREALITY_API

class FARSupportInterface ;

/**
 * A result of an intersection found during a hit-test.
 */
USTRUCT( BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
struct FARTraceResult
{
	GENERATED_BODY();
	
	UE_API FARTraceResult();
	
	UE_API FARTraceResult( const TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe>& InARSystem, float InDistanceFromCamera, EARLineTraceChannels InTraceChannel, const FTransform& InLocalToTrackingTransform, UARTrackedGeometry* InTrackedGeometry );
	
	UE_API float GetDistanceFromCamera() const;

	UE_API void SetLocalToWorldTransform(const FTransform& LocalToWorldTransform);
	
	UE_API FTransform GetLocalToTrackingTransform() const;
	
	UE_API FTransform GetLocalToWorldTransform() const;
	
	UE_API FTransform GetLocalTransform() const;
	
	UE_API UARTrackedGeometry* GetTrackedGeometry() const;
	
	UE_API EARLineTraceChannels GetTraceChannel() const;

	struct FARTraceResultComparer
	{
		inline bool operator()(const FARTraceResult& A, const FARTraceResult& B) const
		{
			return A.GetDistanceFromCamera() < B.GetDistanceFromCamera();
		}
	};
	
private:
	
	/** Distance (in Unreal Units) between the camera and the point where the line trace contacted tracked geometry. */
	UPROPERTY()
	float DistanceFromCamera;
	
	/** The trace channel that generated this trace result. (used for filtering) */
	UPROPERTY()
	EARLineTraceChannels TraceChannel;
	
	/**
	 * The transformation matrix that defines the intersection's rotation, translation and scale
	 * in AR system's local space.
	 */
	UPROPERTY()
	FTransform LocalTransform;
	
	/**
	 * A pointer to the geometry data that was intersected by this trace, if any.
	 */
	UPROPERTY()
	TObjectPtr<UARTrackedGeometry> TrackedGeometry;
	
	/** A reference to the AR system that creates this hit test result. */
	TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> ARSystem;
};

UCLASS()
class UARTraceResultDummy : public UObject
{
	GENERATED_BODY()
};

#undef UE_API
