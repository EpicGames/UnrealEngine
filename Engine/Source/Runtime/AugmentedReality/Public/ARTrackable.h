// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ARTypes.h"
#include "ARComponent.h"
#include "ARTrackable.generated.h"

#define UE_API AUGMENTEDREALITY_API

class FARSupportInterface;
class UAREnvironmentCaptureProbeTexture;
class UMRMeshComponent;

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedGeometry : public UObject
{
	GENERATED_BODY()
	
public:
	UE_API UARTrackedGeometry();
	
	UE_API void InitializeNativeResource(IARRef* InNativeResource);

	UE_API virtual void DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const;

	UE_API void GetNetworkPayload(FARMeshUpdatePayload& Payload);

	UE_API void UpdateTrackedGeometryNoMove(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp);
	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform );
	
	UE_API void UpdateTrackingState( EARTrackingState NewTrackingState );
	
	UE_API void UpdateAlignmentTransform( const FTransform& NewAlignmentTransform );

	UE_API void SetDebugName( FName InDebugName );
	
	UE_API void SetName(const FString& InName);

	UE_API IARRef* GetNativeResource();
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API FTransform GetLocalToWorldTransform() const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API FTransform GetLocalToTrackingTransform() const;
	
	UE_API FTransform GetLocalToTrackingTransform_NoAlignment() const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API EARTrackingState GetTrackingState() const;
	UE_API void SetTrackingState(EARTrackingState NewState);

	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API bool IsTracked() const;

	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API FName GetDebugName() const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API const FString& GetName() const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API int32 GetLastUpdateFrameNumber() const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API float GetLastUpdateTimestamp() const;
	inline void SetLastUpdateTimestamp(double InTimestamp) { LastUpdateTimestamp = InTimestamp; }

	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Tracked Geometry")
	UE_API UMRMeshComponent* GetUnderlyingMesh();
	UE_API void SetUnderlyingMesh(UMRMeshComponent* InMRMeshComponent);

	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Tracked Geometry")
	FGuid UniqueId;

	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Scene Understanding")
	EARObjectClassification GetObjectClassification() const { return ObjectClassification; }
	void SetObjectClassification(EARObjectClassification InClassification) { ObjectClassification = InClassification; }
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Scene Understanding")
	bool HasSpatialMeshUsageFlag(const EARSpatialMeshUsageFlags InFlag) const { return ((int32)SpatialMeshUsageFlags & (int32)InFlag) != 0; }
	void SetSpatialMeshUsageFlags(const EARSpatialMeshUsageFlags InFlags) { SpatialMeshUsageFlags = InFlags; }

protected:
	UE_API TSharedPtr<FARSupportInterface , ESPMode::ThreadSafe> GetARSystem() const;
	UE_API void UpdateSessionPayload(FARSessionPayload& Payload) const;
	
	UPROPERTY()
	FTransform LocalToTrackingTransform;
	
	UPROPERTY()
	FTransform LocalToAlignedTrackingTransform;
	
	UPROPERTY()
	EARTrackingState TrackingState;

	/** A pointer to the native resource in the native AR system */
	TUniquePtr<IARRef> NativeResource;
	
	/** For AR systems that support arbitrary mesh geometry associated with a tracked point */
	UPROPERTY(Transient)
	TObjectPtr<UMRMeshComponent> UnderlyingMesh;

	/** What the scene understanding system thinks this object is */
	UPROPERTY()
	EARObjectClassification ObjectClassification;
	
	/** How the scene understanding system thinks this mesh should be displayed */
	UPROPERTY()
	EARSpatialMeshUsageFlags SpatialMeshUsageFlags;

private:
	TWeakPtr<FARSupportInterface , ESPMode::ThreadSafe> ARSystem;
	
	/** The frame number this tracked geometry was last updated on */
	UPROPERTY()
	int32 LastUpdateFrameNumber;
	
	/** The time reported by the AR system that this object was last updated */
	double LastUpdateTimestamp;
	
	/** A unique name that can be used to identify the anchor for debug purposes */
	UPROPERTY()
	FName DebugName;
	
	/** A descriptive name for the anchor */
	FString AnchorName;
};

UCLASS(MinimalAPI, BlueprintType)
class UARPlaneGeometry : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:

	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FVector InCenter, const FVector InExtent );
	
	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FVector InCenter, const FVector InExtent, const TArray<FVector>& InBoundingPoly, UARPlaneGeometry* InSubsumedBy);
	
	UE_API virtual void DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;

	UE_API void GetNetworkPayload(FARPlaneUpdatePayload& Payload);

public:
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Plane Geometry")
	FVector GetCenter() const { return Center; }
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Plane Geometry")
	FVector GetExtent() const { return Extent; }
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Plane Geometry")
	TArray<FVector> GetBoundaryPolygonInLocalSpace() const { return BoundaryPolygon; }

	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Plane Geometry")
	UARPlaneGeometry* GetSubsumedBy() const { return SubsumedBy; };

	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Plane Geometry")
	EARPlaneOrientation GetOrientation() const { return Orientation; }
	void SetOrientation(EARPlaneOrientation InOrientation) { Orientation = InOrientation; }

private:
	UPROPERTY()
	EARPlaneOrientation Orientation;

	UPROPERTY()
	FVector Center;
	
	UPROPERTY()
	FVector Extent;
	
	UPROPERTY()
	TArray<FVector> BoundaryPolygon;

	// Used by ARCore Only
	UPROPERTY()
	TObjectPtr<UARPlaneGeometry> SubsumedBy = nullptr;
};

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedPoint : public UARTrackedGeometry
{
	GENERATED_BODY()

public:
	UE_API virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;

	UE_API void GetNetworkPayload(FARPointUpdatePayload& Payload);

	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform);
};

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedImage : public UARTrackedGeometry
{
	GENERATED_BODY()

public:
	UE_API virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;

	UE_API void GetNetworkPayload(FARImageUpdatePayload& Payload);

	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector2D InEstimatedSize, UARCandidateImage* InDetectedImage);

	/** @see DetectedImage */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	UARCandidateImage* GetDetectedImage() const { return DetectedImage; };
	
	/*
	 * Get the estimate size of the detected image, where X is the estimated width, and Y is the estimated height.
	 *
	 * Note that ARCore can return a valid estimate size of the detected image when the tracking state of the UARTrackedImage
	 * is tracking. The size should reflect the actual size of the image target, which could be different than the input physical
	 * size of the candidate image.
	 *
	 * ARKit will return the physical size of the ARCandidate image.
	 */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	UE_API FVector2D GetEstimateSize();

protected:
	/** The candidate image that was detected in the scene */
	UPROPERTY()
	TObjectPtr<UARCandidateImage> DetectedImage;

	/** The estimated image size that was detected in the scene */
	UPROPERTY()
	FVector2D EstimatedSize;
};

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedQRCode :
	public UARTrackedImage
{
	GENERATED_BODY()

public:
	UE_API void GetNetworkPayload(FARQRCodeUpdatePayload& Payload);

	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector2D InEstimatedSize, const FString& CodeData, int32 InVersion);

	/** The encoded information in the qr code */
	UPROPERTY(BlueprintReadOnly, Category="QR Code")
	FString QRCode;

	/** The version of the qr code */
	UPROPERTY(BlueprintReadOnly, Category="QR Code")
	int32 Version;
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARFaceTrackingDirection : uint8
{
	/** Blend shapes are tracked as if looking out of the face, e.g. right eye is the mesh's right eye and left side of screen if facing you */
	FaceRelative,
	/** Blend shapes are tracked as if looking at the face, e.g. right eye is the mesh's left eye and right side of screen if facing you (like a mirror) */
	FaceMirrored
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARFaceBlendShape : uint8
{
	// Left eye blend shapes
	EyeBlinkLeft,
	EyeLookDownLeft,
	EyeLookInLeft,
	EyeLookOutLeft,
	EyeLookUpLeft,
	EyeSquintLeft,
	EyeWideLeft,
	// Right eye blend shapes
	EyeBlinkRight,
	EyeLookDownRight,
	EyeLookInRight,
	EyeLookOutRight,
	EyeLookUpRight,
	EyeSquintRight,
	EyeWideRight,
	// Jaw blend shapes
	JawForward,
	JawLeft,
	JawRight,
	JawOpen,
	// Mouth blend shapes
	MouthClose,
	MouthFunnel,
	MouthPucker,
	MouthLeft,
	MouthRight,
	MouthSmileLeft,
	MouthSmileRight,
	MouthFrownLeft,
	MouthFrownRight,
	MouthDimpleLeft,
	MouthDimpleRight,
	MouthStretchLeft,
	MouthStretchRight,
	MouthRollLower,
	MouthRollUpper,
	MouthShrugLower,
	MouthShrugUpper,
	MouthPressLeft,
	MouthPressRight,
	MouthLowerDownLeft,
	MouthLowerDownRight,
	MouthUpperUpLeft,
	MouthUpperUpRight,
	// Brow blend shapes
	BrowDownLeft,
	BrowDownRight,
	BrowInnerUp,
	BrowOuterUpLeft,
	BrowOuterUpRight,
	// Cheek blend shapes
	CheekPuff,
	CheekSquintLeft,
	CheekSquintRight,
	// Nose blend shapes
	NoseSneerLeft,
	NoseSneerRight,
	TongueOut,
	// Treat the head rotation as curves for LiveLink support
	HeadYaw,
	HeadPitch,
	HeadRoll,
	// Treat eye rotation as curves for LiveLink support
	LeftEyeYaw,
	LeftEyePitch,
	LeftEyeRoll,
	RightEyeYaw,
	RightEyePitch,
	RightEyeRoll,
	MAX
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EAREye : uint8
{
	LeftEye,
	RightEye
};

typedef TMap<EARFaceBlendShape, float> FARBlendShapeMap;

UCLASS(MinimalAPI, BlueprintType)
class UARFaceGeometry : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:
	UE_API void UpdateFaceGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InTransform, const FTransform& InAlignmentTransform, FARBlendShapeMap& InBlendShapes, TArray<FVector>& InVertices, const TArray<int32>& Indices, TArray<FVector2D>& InUVs, const FTransform& InLeftEyeTransform, const FTransform& InRightEyeTransform, const FVector& InLookAtTarget);
	
	UE_API virtual void DebugDraw( UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;
	
	UE_API void GetNetworkPayload(FARFaceUpdatePayload& Payload);

public:
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Face Geometry")
	UE_API float GetBlendShapeValue(EARFaceBlendShape BlendShape) const;
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Face Geometry")
	UE_API const TMap<EARFaceBlendShape, float> GetBlendShapes() const;

	const FARBlendShapeMap& GetBlendShapesRef() const { return BlendShapes; }
	
	const TArray<FVector>& GetVertexBuffer() const { return VertexBuffer; }
	const TArray<int32>& GetIndexBuffer() const { return IndexBuffer; }
	const TArray<FVector2D>& GetUVs() const { return UVs; }
	
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Face Geometry")
	UE_API const FTransform& GetLocalSpaceEyeTransform(EAREye Eye) const;

	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Face Geometry")
	UE_API FTransform GetWorldSpaceEyeTransform(EAREye Eye) const;

	/** The target the eyes are looking at */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Face Geometry")
	FVector LookAtTarget;

	UE_DEPRECATED(4.21, "This property is now deprecated, please use GetTrackingState() and check for EARTrackingState::Tracking or IsTracked() instead.")
	/** Whether the face is currently being tracked by the AR system */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Face Geometry", meta=(Deprecated, DeprecationMessage = "This property is now deprecated, please use GetTrackingState() and check for EARTrackingState::Tracking or IsTracked() instead."))
	bool bIsTracked;

private:
	UPROPERTY()
	TMap<EARFaceBlendShape, float> BlendShapes;
	
	// Holds the face data for one or more face components that want access
	TArray<FVector> VertexBuffer;
	TArray<int32> IndexBuffer;
	// @todo JoeG - route the uvs in
	TArray<FVector2D> UVs;

	/** The transform for the left eye */
	UPROPERTY()
	FTransform LeftEyeTransform;
	
	/** The transform for the right eye */
	UPROPERTY()
	FTransform RightEyeTransform;
};

/** A tracked environment texture probe that gives you a cube map for reflections */
UCLASS(MinimalAPI, BlueprintType)
class UAREnvironmentCaptureProbe :
	public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:
	UE_API UAREnvironmentCaptureProbe();
	
	/** Draw a box visulizing the bounds of the probe */
	UE_API virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;
	
	UE_API void GetNetworkPayload(FAREnvironmentProbeUpdatePayload& Payload);

	UE_API void UpdateEnvironmentCapture(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, FVector InExtent);

	/** @see Extent */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Environment Capture Probe")
	UE_API FVector GetExtent() const;
	/** @see EnvironmentCaptureTexture */
	UFUNCTION(BlueprintPure, Category="AR AugmentedReality|Environment Capture Probe")
	UE_API UAREnvironmentCaptureProbeTexture* GetEnvironmentCaptureTexture();

protected:
	/** The size of area this probe covers */
	UPROPERTY()
	FVector Extent;

	/** The cube map of the reflected environment */
	UPROPERTY()
	TObjectPtr<UAREnvironmentCaptureProbeTexture> EnvironmentCaptureTexture;
};

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedObject : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:
	UE_API virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;
	
	UE_API void GetNetworkPayload(FARObjectUpdatePayload& Payload);

	UE_API void UpdateTrackedGeometry(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, UARCandidateObject* InDetectedObject);
	
	/** @see DetectedObject */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Object Detection")
	UARCandidateObject* GetDetectedObject() const { return DetectedObject; };
	
private:
	/** The candidate object that was detected in the scene */
	UPROPERTY()
	TObjectPtr<UARCandidateObject> DetectedObject;
};

UCLASS(MinimalAPI, BlueprintType)
class UARTrackedPose : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:
	UE_API virtual void DebugDraw(UWorld* World, const FLinearColor& OutlineColor, float OutlineThickness, float PersistForSeconds = 0.0f) const override;
	
	UE_API void GetNetworkPayload(FARPoseUpdatePayload& Payload);

	UE_API void UpdateTrackedPose(const TSharedRef<FARSupportInterface , ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform, const FARPose3D& InTrackedPose);
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Pose Tracking")
	const FARPose3D& GetTrackedPoseData() const { return TrackedPose; };
	
private:
	/** The detailed info of the tracked pose */
	UPROPERTY()
	FARPose3D TrackedPose;
};

UCLASS(MinimalAPI, BlueprintType)
class UARMeshGeometry : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:

	UE_API void GetNetworkPayload(FARMeshUpdatePayload& Payload);

	/**
	 * Try to determine the classification of the object at a world space location
	 * @InWorldLocation: the world location where the classification is needed
	 * @OutClassification: the classification result
	 * @OutClassificationLocation: the world location at where the classification is calculated
	 * @MaxLocationDiff: the max distance between the specified world location and the classification location
	 * @return: whether a valid classification result is calculated
	 */
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Classification")
	virtual bool GetObjectClassificationAtLocation(const FVector& InWorldLocation, EARObjectClassification& OutClassification, FVector& OutClassificationLocation, float MaxLocationDiff = 10.f) { return false; }
};


UCLASS(MinimalAPI, BlueprintType)
class UARGeoAnchor : public UARTrackedGeometry
{
	GENERATED_BODY()
	
public:
	UE_API void UpdateGeoAnchor(const TSharedRef<FARSupportInterface, ESPMode::ThreadSafe>& InTrackingSystem, uint32 FrameNumber, double Timestamp, const FTransform& InLocalToTrackingTransform, const FTransform& InAlignmentTransform,
						 float InLongitude, float InLatitude, float InAltitudeMeters, EARAltitudeSource InAltitudeSource);
	
	UE_API void GetNetworkPayload(FARGeoAnchorUpdatePayload& Payload);
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Geo Tracking")
	float GetLongitude() const { return Longitude; }
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Geo Tracking")
	float GetLatitude() const { return Latitude; }
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Geo Tracking")
	float GetAltitudeMeters() const { return AltitudeMeters; }
	
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Geo Tracking")
	EARAltitudeSource GetAltitudeSource() const { return AltitudeSource; }
	
private:
	float Longitude = 0.f;
	float Latitude = 0.f;
	float AltitudeMeters = 0.f;
	EARAltitudeSource AltitudeSource = EARAltitudeSource::Unknown;
};

#undef UE_API
