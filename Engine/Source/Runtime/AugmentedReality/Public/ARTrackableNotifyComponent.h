// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ARTrackable.h"
#include "Components/ActorComponent.h"
#include "ARTrackableNotifyComponent.generated.h"

#define UE_API AUGMENTEDREALITY_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackableDelegate, UARTrackedGeometry*, TrackedGeometry);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackablePlaneDelegate, UARPlaneGeometry*, TrackedPlane);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackablePointDelegate, UARTrackedPoint*, TrackedPoint);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackableImageDelegate, UARTrackedImage*, TrackedImage);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackableFaceDelegate, UARFaceGeometry*, TrackedFace);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackableEnvProbeDelegate, UAREnvironmentCaptureProbe*, TrackedEnvProbe);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTrackableObjectDelegate, UARTrackedObject*, TrackedObject);

/** Component used to listen to ar trackable object changes */
UCLASS(MinimalAPI, meta=(BlueprintSpawnableComponent))
class UARTrackableNotifyComponent :
	public UActorComponent
{
	GENERATED_BODY()

public:
// Base class catch all
	/** Event that happens when any new trackable ar item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableDelegate OnAddTrackedGeometry;

	/** Event that happens when any trackable ar item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableDelegate OnUpdateTrackedGeometry;

	/** Event that happens when any trackable ar item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableDelegate OnRemoveTrackedGeometry;

// Tracked plane events
	/** Event that happens when any new ar plane item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePlaneDelegate OnAddTrackedPlane;

	/** Event that happens when any ar plane item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePlaneDelegate OnUpdateTrackedPlane;

	/** Event that happens when any ar plane item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePlaneDelegate OnRemoveTrackedPlane;

// Tracked Point events
	/** Event that happens when any new ar Point item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePointDelegate OnAddTrackedPoint;

	/** Event that happens when any ar Point item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePointDelegate OnUpdateTrackedPoint;

	/** Event that happens when any ar Point item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackablePointDelegate OnRemoveTrackedPoint;

// Tracked Image events
	/** Event that happens when any new ar Image item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableImageDelegate OnAddTrackedImage;

	/** Event that happens when any ar Image item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableImageDelegate OnUpdateTrackedImage;

	/** Event that happens when any ar Image item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableImageDelegate OnRemoveTrackedImage;

// Tracked Face events
	/** Event that happens when any new ar Face item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableFaceDelegate OnAddTrackedFace;

	/** Event that happens when any ar Face item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableFaceDelegate OnUpdateTrackedFace;

	/** Event that happens when any ar Face item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableFaceDelegate OnRemoveTrackedFace;

// Tracked environment capture probe events
	/** Event that happens when any new ar environment capture probe item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableEnvProbeDelegate OnAddTrackedEnvProbe;

	/** Event that happens when any ar environment capture probe item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableEnvProbeDelegate OnUpdateTrackedEnvProbe;

	/** Event that happens when any ar environment capture probe item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableEnvProbeDelegate OnRemoveTrackedEnvProbe;

// Tracked object events
	/** Event that happens when any new ar detected object item is added */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableObjectDelegate OnAddTrackedObject;

	/** Event that happens when any ar detected object item is updated */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableObjectDelegate OnUpdateTrackedObject;

	/** Event that happens when any ar detected object item is removed from the scene */
	UPROPERTY(BlueprintAssignable, Category="Event")
	FTrackableObjectDelegate OnRemoveTrackedObject;

private:
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

	/**
	 * Triggers the delegate type if the object is the desired class
	 *
	 * #return true if the class matched, false otherwise
	 */
	template<typename OBJ_TYPE, typename DELEGATE_TYPE>
	bool ConditionalDispatchEvent(UARTrackedGeometry* Tracked, DELEGATE_TYPE& Delegate);

	/**
	 * Routes the notification to the proper delegate set
	 *
	 * @param Added the newly added ar trackable
	 */
	UE_API void OnTrackableAdded(UARTrackedGeometry* Added);
	/**
	 * Routes the notification to the proper delegate set
	 *
	 * @param Updated the updated ar trackable
	 */
	UE_API void OnTrackableUpdated(UARTrackedGeometry* Updated);
	/**
	 * Routes the notification to the proper delegate set
	 *
	 * @param Removed the removed ar trackable
	 */
	UE_API void OnTrackableRemoved(UARTrackedGeometry* Removed);
};

#undef UE_API
