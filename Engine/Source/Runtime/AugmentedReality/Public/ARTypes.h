// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataAsset.h"
#include "HAL/ThreadSafeBool.h"
#include "ARTypes.generated.h"

#define UE_API AUGMENTEDREALITY_API

class USceneComponent;
class UARPin;
class UARTrackedGeometry;
class UARLightEstimate;
struct FARTraceResult;
class UTexture2D;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackableAdded, UARTrackedGeometry*);
typedef FOnTrackableAdded::FDelegate FOnTrackableAddedDelegate;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackableUpdated, UARTrackedGeometry*);
typedef FOnTrackableUpdated::FDelegate FOnTrackableUpdatedDelegate;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTrackableRemoved, UARTrackedGeometry*);
typedef FOnTrackableRemoved::FDelegate FOnTrackableRemovedDelegate;


UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARTrackingState : uint8
{
	/** Unknown tracking state */
	Unknown,
	
	/** Currently tracking. */
	Tracking,
	
	/** Currently not tracking, but may resume tracking later. */
	NotTracking,
	
	/** Stopped tracking forever. */
	StoppedTracking
};


UENUM(BlueprintType, Category = "AR AugmentedReality", meta = (Experimental))
enum class EARCaptureType : uint8
{
	/** Camera Capture */
	Camera,

	/** QR Code Capture. */
	QRCode,

	/** Spatial mapping so the app can selectively turn off discovering surfaces */
	SpatialMapping,

	/** Capture detailed information about the scene with all surfaces like walls, floors and so on*/
	SceneUnderstanding,

	/** Capture a mesh around the player's hands */
	HandMesh,
};


/**
 * Channels that let users select which kind of tracked geometry to trace against.
 */
UENUM( BlueprintType, Category="AR AugmentedReality|Trace Result", meta=(Bitflags) )
enum class EARLineTraceChannels : uint8
{
	None = 0,
	
	/** Trace against points that the AR system considers significant . */
	FeaturePoint = 1,
	
	/** Trace against estimated plane that does not have an associated tracked geometry. */
	GroundPlane = 2,
	
	/** Trace against any plane tracked geometries using Center and Extent. */
	PlaneUsingExtent = 4,
	
	/** Trace against any plane tracked geometries using the boundary polygon. */
	PlaneUsingBoundaryPolygon = 8
};
ENUM_CLASS_FLAGS(EARLineTraceChannels);



UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARTrackingQuality : uint8
{
	/** The tracking quality is not available. */
	NotTracking,
	
	/** The tracking quality is limited, relying only on the device's motion. */
	OrientationOnly,
	
	/** The tracking quality is good. */
	OrientationAndPosition
};

UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARTrackingQualityReason : uint8
{
	/** Current Tracking is not limited */
	None,
	
	/** The AR session has not yet gathered enough camera or motion data to provide tracking information. */
	Initializing,
	
	/** The AR session is attempting to resume after an interruption. */
	Relocalizing,
	
	/** The device is moving too fast for accurate image-based position tracking. */
	ExcessiveMotion,
	
	/** The scene visible to the camera does not contain enough distinguishable features for image-based position tracking. */
	InsufficientFeatures,

	/** Tracking lost due to poor lighting conditions. Please move to a more brightly lit area */
	InsufficientLight,

	/** Tracking lost due to bad internal state. Please try restarting the AR experience. */
	 BadState
};

/**
 * Describes the current status of the AR session.
 */
UENUM(BlueprintType, meta=(ScriptName="ARSessionStatusType"))
enum class EARSessionStatus : uint8
{
	/** Unreal AR session has not started yet.*/
	NotStarted,
	/** Unreal AR session is running. */
	Running,
	/** Unreal AR session failed to start due to the AR subsystem not being supported by the device. */
	NotSupported,
	/** The AR session encountered fatal error; the developer should call `StartARSession()` to re-start the AR subsystem. */
	FatalError,
	/** AR session failed to start because it lacks the necessary permission (likely access to the camera or the gyroscope). */
	PermissionNotGranted,
	/** AR session failed to start because the configuration isn't supported. */
	UnsupportedConfiguration,
	/** Session isn't running due to unknown reason; @see FARSessionStatus::AdditionalInfo for more information */
	Other,
};

/** Gives feedback as to whether the AR data can be saved and relocalized or not */
UENUM(BlueprintType, Category="AR AugmentedReality", meta=(Experimental))
enum class EARWorldMappingState : uint8
{
	/** World mapping is not available */
	NotAvailable,
	/** World mapping is still in progress but without enough data for relocalization */
	StillMappingNotRelocalizable,
	/** World mapping is still in progress but there is enough data captured for relocalization */
	StillMappingRelocalizable,
	/** World mapping has mapped the area and is fully relocalizable */
	Mapped
};

/** Describes the tracked plane orientation */
UENUM(BlueprintType)
enum class EARPlaneOrientation : uint8
{
	Horizontal,
	Vertical,
	/** For AR systems that can match planes to slopes */
	Diagonal,
};

/** Indicates what type of object the scene understanding system thinks it is */
UENUM(BlueprintType)
enum class EARObjectClassification : uint8
{
	/** Not applicable to scene understanding */
	NotApplicable,
	/** Scene understanding doesn't know what this is */
	Unknown,
	/** A vertical plane that is a wall */
	Wall,
	/** A horizontal plane that is the ceiling */
	Ceiling,
	/** A horizontal plane that is the floor */
	Floor,
	/** A horizontal plane that is a table */
	Table,
	/** A horizontal plane that is a seat */
	Seat,
	/** A human face */
	Face,
	/** A recognized image in the scene */
	Image,
	/** A chunk of mesh that does not map to a specific object type but is seen by the AR system */
	World,
	/** A closed mesh that was identified in the scene */
	SceneObject,
	/** A user's hand */
	HandMesh,
	/** A door */
	Door,
	/** A window */
	Window,
	// Add other types here...
};

/** Indicates how the spatial mesh should be visualized */
UENUM(BlueprintType)
enum class EARSpatialMeshUsageFlags : uint8
{
	/** Not applicable to scene understanding. */
	NotApplicable = 0,
	/** This mesh should have a visible material applied to it. */
	Visible = 1 << 0,
	/** This mesh should be used when placing objects on real world surfaces.  This must be set to use physics with this mesh. */
	Collision = 1 << 1
};

/** Describes the potential spaces in which the join transform can be defined with AR pose tracking */
UENUM(BlueprintType)
enum class EARJointTransformSpace : uint8
{
	/**
	 * Joint transform is relative to the origin of the model space
	 * which is usually attached to a particular joint
	 * such as the hip
	 */
	Model,
	
	/**
	* Joint transform is relative to its parent
	*/
	ParentJoint,
};

UENUM(BlueprintType)
enum class EARAltitudeSource : uint8
{
	// The framework sets the altitude using a high-resolution digital-elevation model.
	Precise,
	
	// The framework sets the altitude using a coarse digital-elevation model.
	Coarse,
	
	// The app defines the alitude.
	UserDefined,
	
	// Altitude is not yet set.
	Unknown,
};

/** The current state of the AR subsystem including an optional explanation string. */
USTRUCT(BlueprintType)
struct FARSessionStatus
{
public:
	
	GENERATED_BODY()

	FARSessionStatus()
	:FARSessionStatus(EARSessionStatus::Other)
	{}

	FARSessionStatus(EARSessionStatus InStatus, FString InExtraInfo = FString())
		: AdditionalInfo(InExtraInfo)
		, Status(InStatus)
	{

	}
	
	/** Optional information about the current status of the system. */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Session")
	FString AdditionalInfo;

	/** The current status of the AR subsystem. */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Session")
	EARSessionStatus Status;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnARTrackingStateChanged, EARTrackingState, NewTrackingState);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnARTransformUpdated, const FTransform&, OldToNewTransform );

UCLASS()
class UARTypesDummyClass : public UObject
{
	GENERATED_BODY()
};

/** A reference to a system-level AR object  */
class IARRef
{
public:
	virtual void AddRef() = 0;
	virtual void RemoveRef() = 0;
public:
	virtual ~IARRef() = default;

};

/** Tells the image detection code how to assume the image is oriented */
UENUM(BlueprintType)
enum class EARCandidateImageOrientation : uint8
{
	Landscape,
	Portrait
};

/** An asset that points to an image to be detected in a scene and provides the size of the object in real life */
UCLASS(MinimalAPI, BlueprintType)
class UARCandidateImage :
	public UDataAsset
{
	GENERATED_BODY()

public:

	static UARCandidateImage* CreateNewARCandidateImage(UTexture2D* InCandidateTexture, FString InFriendlyName, float InPhysicalWidth, float InPhysicalHeight, EARCandidateImageOrientation InOrientation)
	{
		UARCandidateImage* NewARCandidateImage = NewObject<UARCandidateImage>();
		NewARCandidateImage->CandidateTexture = InCandidateTexture;
		NewARCandidateImage->FriendlyName = InFriendlyName;
		NewARCandidateImage->Width = InPhysicalWidth;
		NewARCandidateImage->Height = InPhysicalHeight;
		NewARCandidateImage->Orientation = InOrientation;

		return NewARCandidateImage;
	}

	/** @see CandidateTexture */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	UTexture2D* GetCandidateTexture() const { return CandidateTexture; }

	/** @see FriendlyName */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	const FString& GetFriendlyName() const { return FriendlyName; }

	/** @see Width */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	float GetPhysicalWidth() const { return Width; }

	/** @see Height */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	float GetPhysicalHeight() const { return Height; }

	/** @see Orientation */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Image Detection")
	EARCandidateImageOrientation GetOrientation() const { return Orientation; }

protected:
#if WITH_EDITOR
	/** Used to enforce physical sizes matching the aspect ratio of the images */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
	/** The image to detect in scenes */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	TObjectPtr<UTexture2D> CandidateTexture;

	/** The friendly name to report back when the image is detected in scenes */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	FString FriendlyName;

	/** The physical width in centimeters of the object that this candidate image represents */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	float Width;

	/** The physical height in centimeters of the object that this candidate image represents. Ignored in ARCore */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	float Height;

	/** The orientation to treat the candidate image as. Ignored in ARCore */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	EARCandidateImageOrientation Orientation;
};

/** An asset that points to an object to be detected in a scene */
UCLASS(MinimalAPI, BlueprintType)
class UARCandidateObject :
	public UDataAsset
{
	GENERATED_BODY()
	
public:
	/** @see CandidateObjectData */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Object Detection")
	const TArray<uint8>& GetCandidateObjectData() const { return CandidateObjectData; }

	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Object Detection")
	void SetCandidateObjectData(const TArray<uint8>& InCandidateObject) { CandidateObjectData = InCandidateObject; }

	/** @see FriendlyName */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Object Detection")
	const FString& GetFriendlyName() const { return FriendlyName; }
	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Object Detection")
	void SetFriendlyName(const FString& NewName) { FriendlyName = NewName; }

	/** @see BoundingBox */
	UFUNCTION(BlueprintPure, Category = "AR AugmentedReality|Object Detection")
	const FBox& GetBoundingBox() const { return BoundingBox; }

	UFUNCTION(BlueprintCallable, Category = "AR AugmentedReality|Object Detection")
	void SetBoundingBox(const FBox& InBoundingBox) { BoundingBox = InBoundingBox; }

private:
	/** The object to detect in scenes */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Object")
	TArray<uint8> CandidateObjectData;
	
	/** The friendly name to report back when the object is detected in scenes */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Object")
	FString FriendlyName;
	
	/** The physical bounds in centimeters of the object that this candidate object represents */
	UPROPERTY(EditAnywhere, Category = "AR Candidate Image")
	FBox BoundingBox;
};

/**
 * Base class for async AR requests
 */
class FARAsyncTask
{
public:
	virtual ~FARAsyncTask() = default;
	
	/** @return whether the task succeeded or not */
	UE_API bool HadError() const;
	/** @return information about the error if there was one */
	UE_API FString GetErrorString() const;
	/** @return whether the task has completed or not */
	UE_API bool IsDone() const;
	
protected:
	FThreadSafeBool bIsDone;
	FThreadSafeBool bHadError;
	FString Error;
};

/** Async task that saves the world data into a buffer */
class FARSaveWorldAsyncTask :
	public FARAsyncTask
{
public:
	/** @return the byte array that the world was saved into. Note uses MoveTemp() for efficiency so only valid once */
	UE_API TArray<uint8> GetSavedWorldData();
	
protected:
	TArray<uint8> WorldData;
};

/** Async task that builds a candidate object used for detection from the ar session */
class FARGetCandidateObjectAsyncTask :
	public FARAsyncTask
{
public:
	/** @return the candidate object that you can use for detection later */
	virtual UARCandidateObject* GetCandidateObject() = 0;
};

class FARErrorGetCandidateObjectAsyncTask :
	public FARGetCandidateObjectAsyncTask
{
public:
	FARErrorGetCandidateObjectAsyncTask(FString InError)
	{
		Error = InError;
		bHadError = true;
		bIsDone = true;
	}
	virtual UARCandidateObject* GetCandidateObject() override { return nullptr; }
};

class FARErrorSaveWorldAsyncTask :
	public FARSaveWorldAsyncTask
{
public:
	FARErrorSaveWorldAsyncTask(FString InError)
	{
		Error = InError;
		bHadError = true;
		bIsDone = true;
	}
};

/** A specific AR video format */
USTRUCT(BlueprintType)
struct FARVideoFormat
{
	GENERATED_BODY()

public:
	FARVideoFormat()
		: FPS(0)
		, Width(0)
		, Height(0)
	{
		
	}
	
	FARVideoFormat(int32 InFPS, int32 InWidth, int32 InHeight)
		: FPS(InFPS)
		, Width(InWidth)
		, Height(InHeight)
	{
		
	}

	bool operator==(const FARVideoFormat& Other) const
	{
		return FPS == Other.FPS && Width == Other.Width && Height == Other.Height;
	}
	
	/** The desired or supported number of frames per second for this video format */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="AR AugmentedReality|Session")
	int32 FPS;
	
	/** The desired or supported width in pixels for this video format */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AR AugmentedReality|Session")
	int32 Width;

	/** The desired or supported height in pixels for this video format */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "AR AugmentedReality|Session")
	int32 Height;

	bool IsValidFormat() { return FPS > 0 && Width > 0 && Height > 0; }

	friend FArchive& operator<<(FArchive& Ar, FARVideoFormat& Format)
	{
		return Ar << Format.FPS << Format.Width << Format.Height;
	}
};

/** Represents a hierarchy of a human pose skeleton tracked by the AR system */
USTRUCT(BlueprintType)
struct FARSkeletonDefinition
{
	GENERATED_BODY()

public:
	/** How many joints this skeleton has */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Pose Tracking")
	int32 NumJoints = 0;
	
	/** The name of each joint in this skeleton */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<FName> JointNames;

	/** The parent index of each joint in this skeleton */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<int32> ParentIndices;
};

/** Represents a human pose tracked in the 2D space */
USTRUCT(BlueprintType)
struct FARPose2D
{
	GENERATED_BODY()

public:
	/** The definition of this skeleton */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Pose Tracking")
	FARSkeletonDefinition SkeletonDefinition;
	
	/** The location of each joint in 2D normalized space */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<FVector2D> JointLocations;

	/** Flags indicating if each joint is tracked */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<bool> IsJointTracked;
};

/** Represents a human pose tracked in the 3D space */
USTRUCT(BlueprintType)
struct FARPose3D
{
	GENERATED_BODY()

public:
	/** The definition of this skeleton */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Pose Tracking")
	FARSkeletonDefinition SkeletonDefinition;
	
	/** The transform of each join in the model space */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<FTransform> JointTransforms;
	
	/** Flags indicating if each joint is tracked */
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	TArray<bool> IsJointTracked;
	
	UPROPERTY(BlueprintReadOnly, Category = "AR AugmentedReality|Pose Tracking")
	EARJointTransformSpace JointTransformSpace = EARJointTransformSpace::Model;
};


/** AR camera intrinsics */
USTRUCT(BlueprintType)
struct FARCameraIntrinsics
{
	GENERATED_BODY()

public:
	/** Camera image resolution in pixels */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Camera")
	FIntPoint ImageResolution = FIntPoint::ZeroValue;
	
	/** Camera focal length in pixels */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Camera")
	FVector2D FocalLength = FVector2D::ZeroVector;
	
	/** Camera principal point in pixels */
	UPROPERTY(BlueprintReadOnly, Category="AR AugmentedReality|Camera")
	FVector2D PrincipalPoint = FVector2D::ZeroVector;
};

#undef UE_API
